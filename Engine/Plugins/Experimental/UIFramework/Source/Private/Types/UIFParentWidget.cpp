// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/UIFParentWidget.h"
#include "UIFWidget.h"
#include "UIFPlayerComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFParentWidget)

/**
 *
 */
FUIFrameworkParentWidget::FUIFrameworkParentWidget(UUIFrameworkWidget* InWidget)
	: Parent(InWidget)
	, bIsParentAWidget(true)
{
}


FUIFrameworkParentWidget::FUIFrameworkParentWidget(UUIFrameworkPlayerComponent* InPlayer)
	: Parent(InPlayer)
	, bIsParentAWidget(false)
{
}


UUIFrameworkWidget* FUIFrameworkParentWidget::AsWidget() const
{
	return CastChecked<UUIFrameworkWidget>(Parent);
}


UUIFrameworkPlayerComponent* FUIFrameworkParentWidget::AsPlayerComponent() const
{
	return CastChecked<UUIFrameworkPlayerComponent>(Parent);
}

bool FUIFrameworkParentWidget::operator== (const UUIFrameworkWidget* Other) const
{
	return Other == Parent && bIsParentAWidget;
}
