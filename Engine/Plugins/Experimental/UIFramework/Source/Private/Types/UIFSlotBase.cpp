// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/UIFSlotBase.h"
#include "UIFWidget.h"
//#include "UIFManagerSubsystem.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFSlotBase)


/**
 *
 */
void FUIFrameworkSlotBase::AuthoritySetWidget(UUIFrameworkWidget* InWidget)
{
	Widget = InWidget;
	WidgetId = Widget ? Widget->GetWidgetId() : FUIFrameworkWidgetId();
}
