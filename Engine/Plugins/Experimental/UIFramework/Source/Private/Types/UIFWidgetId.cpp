// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/UIFWidgetId.h"
#include "UIFWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFWidgetId)

/**
 *
 */
int64 FUIFrameworkWidgetId::KeyGenerator = 0;

FUIFrameworkWidgetId::FUIFrameworkWidgetId(UUIFrameworkWidget* InOwner)
	: Key(InOwner->GetWidgetId().Key)
{
}

FUIFrameworkWidgetId FUIFrameworkWidgetId::MakeNew()
{
	FUIFrameworkWidgetId Result{ ++KeyGenerator };
	bool bIsInvalidKey = Result.Key == INDEX_NONE || Result.Key == 0;
	ensure(!bIsInvalidKey);
	if (bIsInvalidKey)
	{
		KeyGenerator = 1;
		Result.Key = KeyGenerator;
	}
	return Result;
}

FUIFrameworkWidgetId FUIFrameworkWidgetId::MakeRoot()
{
	return FUIFrameworkWidgetId{0LL};
}
