// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/UIFWidgetTree.h"
#include "Types/UIFWidgetTreeOwner.h"
#include "UIFLog.h"
#include "UIFWidget.h"

#if UE_UIFRAMEWORK_WITH_DEBUG
#include "Components/PanelWidget.h"
#endif

#include "Engine/ActorChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFWidgetTree)

#if UE_UIFRAMEWORK_WITH_DEBUG
namespace UE::UIFramework::Private
{
	TArray<FUIFrameworkWidgetTree*> GTrees;
	void TestWidgetTree()
	{
		for (FUIFrameworkWidgetTree* Tree : GTrees)
		{
			Tree->AuthorityTest();
		}
	}

	void PrintWidgetTree()
	{
		for (FUIFrameworkWidgetTree* Tree : GTrees)
		{
			Tree->LogTree();
		}
	}

	void PrintWidgetsChildren()
	{
		for (FUIFrameworkWidgetTree* Tree : GTrees)
		{
			Tree->LogWidgetsChildren();
		}
	}

	static FAutoConsoleCommand CCmdTestWidgetTree(
		TEXT("UIFramework.TestWidgetTree"),
		TEXT("Test if all containers are properly setup."),
		FConsoleCommandDelegate::CreateStatic(TestWidgetTree),
		ECVF_Cheat);

	static FAutoConsoleCommand CCmdPrintWidgetTree(
		TEXT("UIFramework.PrintWidgetTree"),
		TEXT("Print/Log all the widgets in the widgettree as they are store in the tree."),
		FConsoleCommandDelegate::CreateStatic(PrintWidgetTree),
		ECVF_Cheat);

	static FAutoConsoleCommand CCmdPrintWidgetsChildren(
		TEXT("UIFramework.PrintWidgetsChildren"),
		TEXT("Print/Log all the widgets in the treetree as they are used at runtime."),
		FConsoleCommandDelegate::CreateStatic(PrintWidgetsChildren),
		ECVF_Cheat);
}
#endif

/**
 *
 */
FUIFrameworkWidgetTreeEntry::FUIFrameworkWidgetTreeEntry(UUIFrameworkWidget* InParent, UUIFrameworkWidget* InChild)
	: Parent(InParent)
	, Child(InChild)
	, ParentId(InParent ? InParent->GetWidgetId() : FUIFrameworkWidgetId::MakeRoot())
	, ChildId(InChild->GetWidgetId())
{

}

bool FUIFrameworkWidgetTreeEntry::IsParentValid() const
{
	return (Parent && Parent->GetWidgetId() == ParentId) || ParentId.IsRoot();
}

bool FUIFrameworkWidgetTreeEntry::IsChildValid() const
{
	return Child && Child->GetWidgetId() == ChildId;
}

FString FUIFrameworkWidgetTreeEntry::GetDebugString()
{
	return FString::Printf(TEXT("%s %s"), Parent ? *Parent->GetName() : TEXT("<none>"), Child ? *Child->GetName() : TEXT("<none>"));
}

/**
 *
 */
FUIFrameworkWidgetTree::FUIFrameworkWidgetTree(AActor* InReplicatedOwner, IUIFrameworkWidgetTreeOwner* InOwner)
	: ReplicatedOwner(InReplicatedOwner)
	, Owner(InOwner)
{
	check(Owner);
#if UE_UIFRAMEWORK_WITH_DEBUG
	UE::UIFramework::Private::GTrees.Add(this);
#endif
}

FUIFrameworkWidgetTree::~FUIFrameworkWidgetTree()
{
#if UE_UIFRAMEWORK_WITH_DEBUG
	UE::UIFramework::Private::GTrees.RemoveSingleSwap(this);
#endif
}

void FUIFrameworkWidgetTree::PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize)
{
	for (int32 Index : RemovedIndices)
	{
		FUIFrameworkWidgetTreeEntry& Entry = Entries[Index];
		if (Entry.Child)
		{
			if (ensure(Owner))
			{
				Owner->LocalWidgetRemovedFromTree(Entry);
			}

			Entry.Child->LocalDestroyUMGWidget();
			WidgetByIdMap.Remove(Entry.Child->GetWidgetId());
		}
	}
}

void FUIFrameworkWidgetTree::PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize)
{
	for (int32 Index : AddedIndices)
	{
		FUIFrameworkWidgetTreeEntry& Entry = Entries[Index];
		if (Entry.ParentId.IsValid() && Entry.ChildId.IsValid())
		{
			if (ensure(Owner))
			{
				Owner->LocalWidgetWasAddedToTree(Entry);
			}
			WidgetByIdMap.FindOrAdd(Entry.ChildId) = Entry.Child;
			if (!Entry.ParentId.IsRoot())
			{
				WidgetByIdMap.FindOrAdd(Entry.ParentId) = Entry.Parent;
			}
 			LocalOnWidgetAdded.Broadcast(Entry.Child);
		}
	}
}

void FUIFrameworkWidgetTree::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	for (int32 Index : ChangedIndices)
	{
		// Note these events should only be called when the widget was not constructed and are now constructed.
		FUIFrameworkWidgetTreeEntry& Entry = Entries[Index];
		if (Entry.ParentId.IsValid() && Entry.ChildId.IsValid())
		{
			if (ensure(Owner))
			{
				Owner->LocalWidgetWasAddedToTree(Entry);
			}
			WidgetByIdMap.FindOrAdd(Entry.ChildId) = Entry.Child;
			if (!Entry.ParentId.IsRoot())
			{
				WidgetByIdMap.FindOrAdd(Entry.ParentId) = Entry.Parent;
			}
		}
	}
}

bool FUIFrameworkWidgetTree::ReplicateSubWidgets(UActorChannel* Channel, FOutBunch* Bunch, FReplicationFlags* RepFlags)
{
	bool bWroteSomething = false;
#if DO_CHECK
	TSet<UUIFrameworkWidget*> AllChildren;
#endif
	for (FUIFrameworkWidgetTreeEntry& Entry : Entries)
	{
		UUIFrameworkWidget* Widget = Entry.Child;
		if (IsValid(Widget))
		{
#if DO_CHECK
			bool bAlreadyInSet = false;
			AllChildren.Add(Widget, &bAlreadyInSet);
			ensureMsgf(bAlreadyInSet == false, TEXT("The widget has more than one parent."));
#endif

			bWroteSomething |= Channel->ReplicateSubobject(Widget, *Bunch, *RepFlags);
		}
	}
	return bWroteSomething;
}

void FUIFrameworkWidgetTree::AuthorityAddRoot(UUIFrameworkWidget* Widget)
{
	check(Widget);
	AuthorityAddChildInternal(nullptr, Widget, true);
}

void FUIFrameworkWidgetTree::AuthorityAddWidget(UUIFrameworkWidget* Parent, UUIFrameworkWidget* Child)
{
	check(Parent);
	check(Child);
	AuthorityAddChildInternal(Parent, Child, true);
}

void FUIFrameworkWidgetTree::AuthorityAddChildInternal(UUIFrameworkWidget* Parent, UUIFrameworkWidget* Child, bool bFirst)
{
	if (Child == nullptr)
	{
		return;
	}

	Child->AuthoritySetWidgetTreeOwner(Owner);
	check(bFirst || Parent->GetWidgetTreeOwner() == Owner);

	bool bOuterIsDifferent = Child->GetOuter() != ReplicatedOwner;
	if (bOuterIsDifferent)
	{
		//if (Child->GetOuter() == GetTransientPackage())
		{
			//If the outer is the transient package, then there are no replication owner yet and it safe to rename it with the correct new outer.
			//If the Outer is not the transient, then the widget got replicated with another player.There are no "reset" and we should duplicate and delete the object.
			//For now only do the rename.It works but it is not the best.
 			Child->Rename(nullptr, ReplicatedOwner);
		}
	}

	const FObjectKey ChildKey = Child;
	if (int32* PreviousEntryIndexPtr = AuthorityIndexByWidgetMap.Find(ChildKey))
	{
		check(Entries.IsValidIndex(*PreviousEntryIndexPtr));
		FUIFrameworkWidgetTreeEntry& PreviousEntry = Entries[*PreviousEntryIndexPtr];
		if (PreviousEntry.Parent != Parent)
		{
			// Same child, different parent. Need to build a new entry for replication.
			PreviousEntry = FUIFrameworkWidgetTreeEntry(Parent, Child);
			MarkItemDirty(PreviousEntry);
		}

		if (bOuterIsDifferent)
		{
			AuthorityAddChildRecursiveInternal(Child);
		}
	}
	else
	{
		int32 NewEntryIndex = Entries.Emplace(Parent, Child);

		FUIFrameworkWidgetTreeEntry& NewEntry = Entries[NewEntryIndex];
		MarkItemDirty(NewEntry);

		AuthorityIndexByWidgetMap.Add(ChildKey) = NewEntryIndex;
		WidgetByIdMap.Add(Child->GetWidgetId()) = Child;

		if (ensure(ReplicatedOwner) && ReplicatedOwner->IsUsingRegisteredSubObjectList())
		{
			ReplicatedOwner->AddReplicatedSubObject(Child);
		}

		AuthorityAddChildRecursiveInternal(Child);
		if (bFirst)
		{
			AuthorityOnWidgetAdded.Broadcast(Child);
		}
	}
}

void FUIFrameworkWidgetTree::AuthorityAddChildRecursiveInternal(UUIFrameworkWidget* InParentWidget)
{
	FUIFrameworkWidgetTree* Self = this;
	InParentWidget->AuthorityForEachChildren([Self, InParentWidget](UUIFrameworkWidget* ChildWidget)
		{
			if (ChildWidget != nullptr)
			{
				Self->AuthorityAddChildInternal(InParentWidget, ChildWidget, false);
			}
		});
}

void FUIFrameworkWidgetTree::AuthorityRemoveWidgetAndChildren(UUIFrameworkWidget* Widget)
{
	if (ensure(Widget))
	{
		if (AuthorityRemoveChildRecursiveInternal(Widget))
		{
			MarkArrayDirty();
			AuthorityOnWidgetRemoved.Broadcast(Widget);
		}
	}
}

//void FUIFrameworkWidgetTree::AuthorityReplaceWidget(UUIFrameworkWidget* OldWidget, UUIFrameworkWidget* NewWidget)
//{
//	bool bHasReplaced = false;
//	for (FUIFrameworkWidgetTreeEntry& Entry : Entries)
//	{
//		if (Entry.Child == OldWidget)
//		{
//			bHasReplaced = true;
//		}
//		if (Entry.Parent == OldWidget)
//		{
//			bHasReplaced = true;
//		}
//	}
//
//	if (bHasReplaced)
//	{
//
//	}
//}

void FUIFrameworkWidgetTree::LocalRemoveRoot(const UUIFrameworkWidget* Widget)
{
	if (ensure(Owner && Widget))
	{
		Owner->LocalRemoveWidgetRootFromTree(Widget);
	}
}

bool FUIFrameworkWidgetTree::AuthorityRemoveChildRecursiveInternal(UUIFrameworkWidget* Widget)
{
	if (Widget == nullptr)
	{
		return false;
	}

	Widget->AuthoritySetWidgetTreeOwner(nullptr);
	const FObjectKey WidgetKey = Widget;
	int32 PreviousEntryIndex = INDEX_NONE;
	if (AuthorityIndexByWidgetMap.RemoveAndCopyValue(WidgetKey, PreviousEntryIndex))
	{
		check(Entries.IsValidIndex(PreviousEntryIndex));

		WidgetByIdMap.Remove(Widget->GetWidgetId());

		if (ensure(ReplicatedOwner) && ReplicatedOwner->IsUsingRegisteredSubObjectList())
		{
			ReplicatedOwner->DestroyReplicatedSubObjectOnRemotePeers(Widget);
		}

		Entries.RemoveAtSwap(PreviousEntryIndex);

		// Fix up the swap item
		if (Entries.IsValidIndex(PreviousEntryIndex))
		{
			if (int32* FixUpChildIndexPtr = AuthorityIndexByWidgetMap.Find(FObjectKey(Entries[PreviousEntryIndex].Child)))
			{
				*FixUpChildIndexPtr = PreviousEntryIndex;
			}
		}

		FUIFrameworkWidgetTree* Self = this;
		Widget->AuthorityForEachChildren([Self](UUIFrameworkWidget* ChildWidget)
			{
				if (ChildWidget)
				{
					Self->AuthorityRemoveChildRecursiveInternal(ChildWidget);
				}
			});	

		return true;
	}
	return false;
}

void FUIFrameworkWidgetTree::AuthorityAddAllWidgetsFromActorChannel()
{
	if (ensure(ReplicatedOwner) && ReplicatedOwner->IsUsingRegisteredSubObjectList())
	{
		for (const FUIFrameworkWidgetTreeEntry& Entry : Entries)
		{
			if (Entry.Child)
			{
				ReplicatedOwner->AddReplicatedSubObject(Entry.Child);
			}
		}
	}
}

void FUIFrameworkWidgetTree::AuthorityRemoveAllWidgetsFromActorChannel()
{
	if (ensure(ReplicatedOwner) && ReplicatedOwner->IsUsingRegisteredSubObjectList())
	{
		for (const FUIFrameworkWidgetTreeEntry& Entry : Entries)
		{
			if (Entry.Child && ReplicatedOwner->IsReplicatedSubObjectRegistered(Entry.Child))
			{
				ReplicatedOwner->DestroyReplicatedSubObjectOnRemotePeers(Entry.Child);
			}
		}
	}
}

FUIFrameworkWidgetTreeEntry* FUIFrameworkWidgetTree::LocalGetEntryByReplicationId(int32 ReplicationId)
{
	return Entries.FindByPredicate([ReplicationId](const FUIFrameworkWidgetTreeEntry& Entry)
	{
		return Entry.ReplicationID == ReplicationId;
	});
}

const FUIFrameworkWidgetTreeEntry* FUIFrameworkWidgetTree::LocalGetEntryByReplicationId(int32 ReplicationId) const
{
	return const_cast<FUIFrameworkWidgetTree*>(this)->LocalGetEntryByReplicationId(ReplicationId);
}

UUIFrameworkWidget* FUIFrameworkWidgetTree::FindWidgetById(FUIFrameworkWidgetId WidgetId)
{
	const TWeakObjectPtr<UUIFrameworkWidget>* Found = WidgetByIdMap.Find(WidgetId);
	return Found ? Found->Get() : nullptr;
}

const UUIFrameworkWidget* FUIFrameworkWidgetTree::FindWidgetById(FUIFrameworkWidgetId WidgetId) const
{
	return const_cast<FUIFrameworkWidgetTree*>(this)->FindWidgetById(WidgetId);
}

const FUIFrameworkWidgetTreeEntry* FUIFrameworkWidgetTree::FindRootEntryById(FUIFrameworkWidgetId WidgetId) const
{
	FUIFrameworkWidgetId NextWidgetId = WidgetId;
	while (NextWidgetId.IsValid())
	{
		const FUIFrameworkWidgetTreeEntry* Entry = Entries.FindByPredicate([NextWidgetId](const FUIFrameworkWidgetTreeEntry& Entry)
			{
				return Entry.ChildId == NextWidgetId;
			});

		if (Entry)
		{
			if (Entry->ParentId.IsRoot())
			{
				return Entry;
			}
			else
			{
				NextWidgetId = Entry->ParentId;
			}
		}
		else
		{
			return nullptr;
		}
	}
	return nullptr;
}

#if UE_UIFRAMEWORK_WITH_DEBUG
void FUIFrameworkWidgetTree::AuthorityTest() const
{
	if (ReplicatedOwner == nullptr || !ReplicatedOwner->HasAuthority())
	{
		UE_LOGF(LogUIFramework, Warning, "Can't run the WidgetTree tests on local.");
		return;
	}

	TSet<FUIFrameworkWidgetId> UniqueIds;
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		const FUIFrameworkWidgetTreeEntry& Entry = Entries[Index];

		ensureAlwaysMsgf(Entry.ParentId.IsValid(), TEXT("Invalid ParentId"));
		if (!Entry.ParentId.IsRoot())
		{
			ensureAlwaysMsgf(Entry.Parent, TEXT("Invalid Parent"));
		}
		else
		{
			ensureAlwaysMsgf(!Entry.Parent, TEXT("Valid Parent"));
		}
		ensureAlwaysMsgf(Entry.Child, TEXT("Invalid Child"));
		ensureAlwaysMsgf(Entry.ChildId.IsValid(), TEXT("Invalid ChildId"));

		if (Entry.ChildId.IsValid())
		{
			ensureAlwaysMsgf(!UniqueIds.Contains(Entry.ChildId), TEXT("Duplicated id"));
			UniqueIds.Add(Entry.ChildId);
		}

		if (Entry.Child)
		{
			ensureAlwaysMsgf(Entry.ChildId == Entry.Child->GetWidgetId(), TEXT("Id do not matches"));
		}
		
		if (Entry.Parent)
		{
			bool bFound = false;
			const UUIFrameworkWidget* ToFindWidget = Entry.Child;
			Entry.Parent->AuthorityForEachChildren([&bFound, ToFindWidget](UUIFrameworkWidget* ChildWidget)
				{
					bFound = bFound || ToFindWidget == ChildWidget;
				});

			ensureAlwaysMsgf(bFound, TEXT("Widget is in the tree but not in the AuthorityForEachChildren"));
		}


		const int32* FoundIndexPtr = AuthorityIndexByWidgetMap.Find(FObjectKey(Entry.Child));
		if (FoundIndexPtr)
		{
			int32 FoundIndex = *FoundIndexPtr;
			ensureAlwaysMsgf(FoundIndex == Index, TEXT("Widget index doesn't match what is in the map"));
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Widget no in the map"));
		}
		

		if (Entry.Child)
		{
			ensureAlwaysMsgf(Entry.Child->GetWidgetId().IsValid(), TEXT("The id is not valid."));
			const TWeakObjectPtr<UUIFrameworkWidget>* FoundWidgetPtr = WidgetByIdMap.Find(Entry.ChildId);
			if (FoundWidgetPtr)
			{
				ensureAlwaysMsgf(FoundWidgetPtr->Get(), TEXT("The found widget is invalid"));
				if (UUIFrameworkWidget* Widget = FoundWidgetPtr->Get())
				{
					ensureAlwaysMsgf(Widget == Entry.Child, TEXT("Widget in the map doesn't matches with the entry widget."));
				}
			}
			else
			{
				ensureAlwaysMsgf(false, TEXT("Widget no in the map"));
			}
		}
	}
	UE_LOGF(LogUIFramework, Log, "Completed WidgetTree tests");
}

void FUIFrameworkWidgetTree::LogTreeInternal(const FUIFrameworkWidgetTreeEntry& ParentEntry, FString Spaces) const
{
	Spaces.AppendChar(TEXT('>'));
	for (FUIFrameworkWidgetTreeEntry Entry : Entries)
	{
		if (ParentEntry.ChildId == Entry.ParentId)
		{
			UE_LOGF(LogUIFramework, Log, "%ls%ls", *Spaces, Entry.Child ? *Entry.Child->GetName() : TEXT("<unknow>"));
			LogTreeInternal(Entry, Spaces);
		}
	}
}

void FUIFrameworkWidgetTree::LogTree() const
{
	UE_LOGF(LogUIFramework, Log, "WidgetTree for owner '%ls'.", ReplicatedOwner ? *ReplicatedOwner->GetPathName() : TEXT("<none>"));

	for (FUIFrameworkWidgetTreeEntry Entry : Entries)
	{
		if (Entry.ParentId.IsRoot())
		{
			UE_LOGF(LogUIFramework, Log, ">%ls", Entry.Child ? *Entry.Child->GetName() : TEXT("<unknow>"));
			LogTreeInternal(Entry, TEXT(">"));
		}
	}
}

void FUIFrameworkWidgetTree::AuthorityLogWidgetsChildrenInternal(UUIFrameworkWidget* Widget, FString Spaces) const
{
	Spaces.AppendChar(TEXT('>'));
	Widget->AuthorityForEachChildren([this, &Spaces](UUIFrameworkWidget* Child)
		{
			UE_LOGF(LogUIFramework, Log, "%ls%ls", *Spaces, Child ? *Child->GetName() : TEXT("<unknow>"));
			if (Child)
			{
				AuthorityLogWidgetsChildrenInternal(Child, Spaces);
			}
		});
}

void FUIFrameworkWidgetTree::LocalLogWidgetsChildrenInternal(const UWidget* UMGWidget, FString Spaces) const
{
	Spaces.AppendChar(TEXT('>'));
	if (const UPanelWidget* UMGPanel = Cast<UPanelWidget>(UMGWidget))
	{
		const int32 Count = UMGPanel->GetChildrenCount();
		for (int32 Index = 0; Index < Count; ++Index)
		{
			UWidget* ChildWidget = UMGPanel->GetChildAt(Index);
			UE_LOGF(LogUIFramework, Log, "%ls%ls", *Spaces, ChildWidget ? *ChildWidget->GetName() : TEXT("<unknow>"));
			if (ChildWidget)
			{
				LocalLogWidgetsChildrenInternal(ChildWidget, Spaces);
			}
		}
	}
}

void FUIFrameworkWidgetTree::LogWidgetsChildren() const
{
	if (ReplicatedOwner == nullptr)
	{
		return;
	}
	UE_LOGF(LogUIFramework, Log, "Widgets Child for owner '%ls'.", *ReplicatedOwner->GetPathName());

	if (ReplicatedOwner->HasAuthority())
	{
		UE_LOGF(LogUIFramework, Log, "== Authority ==");

		for (FUIFrameworkWidgetTreeEntry Entry : Entries)
		{
			if (Entry.ParentId.IsRoot())
			{
				UE_LOGF(LogUIFramework, Log, ">%ls", Entry.Child ? *Entry.Child->GetName() : TEXT("<unknow>"));
				if (Entry.Child)
				{
					AuthorityLogWidgetsChildrenInternal(Entry.Child, TEXT(">"));
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogUIFramework, Log, "== Local ==");

		for (FUIFrameworkWidgetTreeEntry Entry : Entries)
		{
			if (Entry.ParentId.IsRoot())
			{
				UWidget* UMGWidget = Entry.Child ? Entry.Child->LocalGetUMGWidget() : nullptr;

				UE_LOGF(LogUIFramework, Log, ">%ls", UMGWidget ? *UMGWidget->GetName() : TEXT("<unknow>"));
				if (UMGWidget)
				{
					LocalLogWidgetsChildrenInternal(UMGWidget, TEXT(">"));
				}
			}
		}
	}
}
#endif
