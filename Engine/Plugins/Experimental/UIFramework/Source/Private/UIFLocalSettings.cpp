// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFLocalSettings.h"

#include "Async/Async.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFLocalSettings)

UUIFrameworkLocalSettings::UUIFrameworkLocalSettings()
{
	ErrorResource = FSoftObjectPath(TEXT("/UIFramework/Materials/T_Icon_Warning_32.T_Icon_Warning_32"));
	LoadingResource = FSoftObjectPath(TEXT("/UIFramework/Materials/M_UI_DefaultLoading.M_UI_DefaultLoading"));
}

void UUIFrameworkLocalSettings::LoadResources() const
{
	if (bResourceLoaded)
	{
		return;
	}
	bResourceLoaded = true;

	const UUIFrameworkLocalSettings* Settings = GetDefault<UUIFrameworkLocalSettings>();
	TArray<FSoftObjectPath> ObjectsToLoad;
	ObjectsToLoad.Reserve(2);
	ObjectsToLoad.Add(Settings->ErrorResource.ToSoftObjectPath());
	ObjectsToLoad.Add(Settings->LoadingResource.ToSoftObjectPath());
	UAssetManager::GetStreamableManager().RequestAsyncLoad(
		ObjectsToLoad,
		[]()
		{
			if (UObjectInitialized() && !IsEngineExitRequested())
			{
				UUIFrameworkLocalSettings* Default = GetMutableDefault<UUIFrameworkLocalSettings>();
				if (Default->ErrorResourcePtr)
				{
					Default->ErrorResourcePtr->RemoveFromRoot();
				}
				Default->ErrorResourcePtr = Default->ErrorResource.Get();
				if (Default->ErrorResourcePtr)
				{
					Default->ErrorResourcePtr->AddToRoot();
				}

				if (Default->LoadingResourcePtr)
				{
					Default->LoadingResourcePtr->RemoveFromRoot();
				}
				Default->LoadingResourcePtr = Default->LoadingResource.Get();
				if (Default->LoadingResourcePtr)
				{
					Default->LoadingResourcePtr->AddToRoot();
				}
			}
		},
		FStreamableManager::DefaultAsyncLoadPriority);
}


FName UUIFrameworkLocalSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
void UUIFrameworkLocalSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	bResourceLoaded = false;
	LoadResources();
}
#endif
