// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFLog.h"

DEFINE_LOG_CATEGORY(LogUIFramework);