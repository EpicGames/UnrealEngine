// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFModule.h"

#include "Modules/ModuleManager.h"
#include "UIFPresenter.h"
#include "UIFWidget.h"

namespace UE::UIFramework::Private
{
	static TSubclassOf<UUIFrameworkPresenter> Director;
}

// UIFrameworkWidget has 
//	1. a WidgetTree owner (UUIFrameworkPlayerComponent/IUIFrameworkWidgetTreeOwner) [authority/local]
//		set by the WidgetTree when they are added or removed from the tree
//  2. a replication Outer / GetOuter (APlayerController or transient if not added yet) [authority/local]
//		set by the WidgetTree when they are added
//  3. a wrapper object (VerseUIWidget) [authority]
//		set by the VerseUIWidget on creation
//	4. a parent widget [authority]
//		set by FUIFrameworkModule::AuthorityAttachWidget or FUIFrameworkModule::AuthorityDetachWidgetFromParent

UUIFrameworkWidget* FUIFrameworkModule::AuthorityAttachWidget(FUIFrameworkParentWidget Parent, UUIFrameworkWidget* Child)
{
	if (ensure(Child && Parent.IsParentValid()))
	{
		// If re-parenting on itself or already attach
		if (Parent == Child || Child->AuthorityGetParent() == Parent)
		{
			return Child;
		}

		// Remove from previous parent.
		if (Child->AuthorityGetParent().IsParentValid())
		{
			bool bHasSameWidgetTree = false;
			if (Parent.IsPlayerComponent())
			{
				bHasSameWidgetTree = &Parent.AsPlayerComponent()->GetWidgetTree() == Child->GetWidgetTree();
			}
			else
			{
				check(Parent.IsWidget());
				bHasSameWidgetTree = Parent.AsWidget()->GetWidgetTree() == Child->GetWidgetTree();
			}
			AuthorityDetachWidgetFromParentInternal(Child, bHasSameWidgetTree);
		}

		Child->AuthorityParent = Parent;

		// If the parent in the WidgetTree or the parent a root, then add it to the WidgetTree
		if (Parent.IsPlayerComponent())
		{
			Parent.AsPlayerComponent()->GetWidgetTree().AuthorityAddRoot(Child);
		}
		else if (FUIFrameworkWidgetTree* WidgetTree = Parent.AsWidget()->GetWidgetTree())
		{
			WidgetTree->AuthorityAddWidget(Child->AuthorityParent.AsWidget(), Child);
		}
	}

	return Child;
}

//void FUIFrameworkModule::AuthoritySetParentReplicationOwnerRecursive(UUIFrameworkWidget* Widget)
//{
	//Widget->AuthorityForEachChildren([Widget](UUIFrameworkWidget* Child)
	//	{
	//		if (Child != nullptr)
	//		{
	//			check(Child->AuthorityGetParent().IsWidget() && Child->AuthorityGetParent().AsWidget() == Widget);
	//			//Child->OwnerPlayerComponent = Widget->OwnerPlayerComponent;
	//			Child->AuthorityParent = FUIFrameworkParentWidget(Widget);
	//			AuthoritySetParentReplicationOwnerRecursive(Child);
	//		}
	//	});
//}

//UUIFrameworkWidget* FUIFrameworkModule::AuthorityRenameRecursive(UUIFrameworkPlayerComponent* ReplicationOwner, UUIFrameworkWidget* Widget, UObject* NewOuter)
//{
	//if (NewOuter != Widget->GetOuter())
	//{
	//	if (Widget->GetOuter() == GetTransientPackage())
	//	{
	//		// If the outer is the transient package, then there are no replication owner yet and it safe to just rename it with the correct new owner.
	//		Widget->Rename(nullptr, NewOuter);
	//	}
	//	else
	//	{
	//		// The widget change owner, we need to create a new one to replicate it correctly.
	//		UUIFrameworkWidget* NewWidget = DuplicateObject<UUIFrameworkWidget>(Widget, NewOuter);
	//		// Replace all instance in the widget tree
	//		ReplicationOwner->GetWidgetTree()->AuthorityReplaceWidget(Widget, NewWidget);
	//		// Notify it's owner
	//		if (Widget->AuthorityGetWrapper())
	//		{
	//			Widget->AuthorityGetWrapper()->ReplaceWidget(Widget, NewWidget);
	//		}
	//		Widget = NewWidget;
	//	}

	//	Widget->AuthorityForEachChildren([ReplicationOwner, NewOuter](UUIFrameworkWidget* Child)
	//		{
	//			if (Child != nullptr)
	//			{
	//				AuthorityRenameRecursive(ReplicationOwner, Child, NewOuter);
	//			}
	//		});
	//}
//	return Widget;
//}

bool FUIFrameworkModule::AuthorityCanWidgetBeAttached(FUIFrameworkParentWidget Parent, UUIFrameworkWidget* Child)
{
	return Child && Parent.IsParentValid() && Parent != Child;
}

void FUIFrameworkModule::AuthorityDetachWidgetFromParent(UUIFrameworkWidget* Child)
{
	AuthorityDetachWidgetFromParentInternal(Child, false);
}

void FUIFrameworkModule::AuthorityDetachWidgetFromParentInternal(UUIFrameworkWidget* Child, bool bTemporary)
{
	check(Child);

	// If it's in the WidgetTree, we need to remove it.
	if (!bTemporary && Child->GetWidgetTreeOwner())
	{
		//bTemporary: we do not want to remove and re-add the same widget if it's not needed. 
		//Removing them would cause the local to recreate them instead of re-parenting them.
		Child->GetWidgetTreeOwner()->GetWidgetTree().AuthorityRemoveWidgetAndChildren(Child);
	}

	// Notify the widget that we removed a child.
	if (Child->AuthorityGetParent().IsParentValid())
	{
		if (Child->AuthorityGetParent().IsWidget())
		{
			Child->AuthorityGetParent().AsWidget()->AuthorityRemoveChild(Child);
		}
		else
		{
			check(Child->AuthorityGetParent().IsPlayerComponent());
			Child->AuthorityGetParent().AsPlayerComponent()->AuthorityRemoveChild(Child);
		}
	}

	Child->AuthorityParent = FUIFrameworkParentWidget();
}

void FUIFrameworkModule::SetPresenterClass(TSubclassOf<UUIFrameworkPresenter> InDirector)
{
	UE::UIFramework::Private::Director = InDirector;
}

TSubclassOf<UUIFrameworkPresenter> FUIFrameworkModule::GetPresenterClass()
{
	return UE::UIFramework::Private::Director.Get() ? UE::UIFramework::Private::Director : TSubclassOf<UUIFrameworkPresenter>(UUIFrameworkGameViewportPresenter::StaticClass());
}

IMPLEMENT_MODULE(FUIFrameworkModule, UIFramework)
