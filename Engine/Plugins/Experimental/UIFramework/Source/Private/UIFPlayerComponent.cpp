// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFPlayerComponent.h"
#include "Misc/MemStack.h"
#include "UIFLog.h"
#include "UIFModule.h"
#include "Types/UIFWidgetOwner.h"
#include "UIFLocalSettings.h"
#include "UIFPresenter.h"
#include "UIFWidget.h"

#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFPlayerComponent)


/**
 *
 */

void FUIFrameworkGameLayerSlotList::PreReplicatedRemove(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkGameLayerSlot& Slot = Entries[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			if (UUIFrameworkPresenter* Presenter = Owner->GetOrCreatePresenter())
			{
				Presenter->RemoveFromViewport(Slot.GetWidgetId());
			}
		}
	}
}

void FUIFrameworkGameLayerSlotList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkGameLayerSlot& Slot = Entries[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			Owner->LocalAddChild(Slot.GetWidgetId());
		}
	}
}

void FUIFrameworkGameLayerSlotList::AddEntry(FUIFrameworkGameLayerSlot Entry)
{
	FUIFrameworkGameLayerSlot& NewEntry = Entries.Add_GetRef(MoveTemp(Entry));
	MarkItemDirty(NewEntry);
}

bool FUIFrameworkGameLayerSlotList::RemoveEntry(UUIFrameworkWidget* Widget)
{
	check(Widget);
	const int32 Index = Entries.IndexOfByPredicate([Widget](const FUIFrameworkGameLayerSlot& Entry){ return Entry.AuthorityGetWidget() == Widget; });
	if (Index != INDEX_NONE)
	{
		Entries.RemoveAt(Index);
		MarkArrayDirty();
	}
	return Index != INDEX_NONE;
}

FUIFrameworkGameLayerSlot* FUIFrameworkGameLayerSlotList::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return Entries.FindByPredicate([WidgetId](const FUIFrameworkGameLayerSlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}

const FUIFrameworkGameLayerSlot* FUIFrameworkGameLayerSlotList::FindEntry(FUIFrameworkWidgetId WidgetId) const
{
	return Entries.FindByPredicate([WidgetId](const FUIFrameworkGameLayerSlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}


/**
 *
 */
UUIFrameworkPlayerComponent::UUIFrameworkPlayerComponent()
	: RootList(this)
	, WidgetTree(GetOwner(), this)
{
	UE_LOGF(LogUIFramework, VeryVerbose, "Constructing UIFrameworkPlayerComponent");
	SetIsReplicatedByDefault(true);
	bWantsInitializeComponent = true;

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_DuringPhysics;
}

void UUIFrameworkPlayerComponent::InitializeComponent()
{
	Super::InitializeComponent();

	UE_LOGF(LogUIFramework, VeryVerbose, "Initializing UIFrameworkPlayerComponent");

	if (GetOwner()->HasAuthority())
	{
		WidgetTree.AuthorityAddAllWidgetsFromActorChannel();
	}
	else
	{
		GetDefault<UUIFrameworkLocalSettings>()->LoadResources();
	}
}

void UUIFrameworkPlayerComponent::UninitializeComponent()
{
	UE_LOGF(LogUIFramework, VeryVerbose, "Uninitializing UIFrameworkPlayerComponent");

	// On local, remove all UWidget.
	if (GetOwner()->HasAuthority())
	{
		WidgetTree.AuthorityRemoveAllWidgetsFromActorChannel();
	}
	else
	{
		for (FUIFrameworkGameLayerSlot& Entry : RootList.Entries)
		{
			if (UUIFrameworkWidget* ChildWidget = GetWidgetTree().FindWidgetById(Entry.GetWidgetId()))
			{
				ChildWidget->LocalDestroyUMGWidget();
			}
		}
		FTSTicker::RemoveTicker(PendingWidgetConstructedTickerHandle);
	}

	for (const auto& ClassesToLoadPair : ClassesToLoad)
	{
		if (ClassesToLoadPair.Value.StreamableHandle.IsValid())
		{
			ClassesToLoadPair.Value.StreamableHandle->CancelHandle();
		}
	}
	ClassesToLoad.Empty();

	Super::UninitializeComponent();
}

void UUIFrameworkPlayerComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, RootList, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, WidgetTree, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, WidgetToFocus, Params);
}

bool UUIFrameworkPlayerComponent::ReplicateSubobjects(UActorChannel* Channel, class FOutBunch* Bunch, FReplicationFlags* RepFlags)
{
	bool bWroteSomething = Super::ReplicateSubobjects(Channel, Bunch, RepFlags);
	if (!bReplicateUsingRegisteredSubObjectList)
	{
		bWroteSomething |= WidgetTree.ReplicateSubWidgets(Channel, Bunch, RepFlags);
	}
	return bWroteSomething;
}

void UUIFrameworkPlayerComponent::AddWidget(FUIFrameworkGameLayerSlot InEntry)
{
	APlayerController* LocalOwner = GetPlayerController();
	check(!LocalOwner || LocalOwner->HasAuthority());

	if (InEntry.AuthorityGetWidget() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToAdd");
	}
	else
	{
		UE_LOG(LogUIFramework, VeryVerbose, TEXT("Attempting to add widget '%" INT64_FMT "' on the server."), InEntry.AuthorityGetWidget()->GetWidgetId().GetKey());
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		InEntry.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, InEntry.AuthorityGetWidget()));
		RootList.AddEntry(InEntry);
	}
}

void UUIFrameworkPlayerComponent::RemoveWidget(UUIFrameworkWidget* Widget)
{
	APlayerController* LocalOwner = GetPlayerController();
	check(!LocalOwner || LocalOwner->HasAuthority());

	if (Widget == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be removed."), ELogVerbosity::Warning, "InvalidWidgetToRemove");
	}
	else
	{
		if (Widget->GetWidgetTreeOwner() != this)
		{
			FFrame::KismetExecutionMessage(TEXT("The widget was not added on this player. It can't be removed on this player."), ELogVerbosity::Warning, "InvalidPlayerParentOnRemovedWidget");
		}
		else
		{
			UE_LOG(LogUIFramework, VeryVerbose, TEXT("Attempting to remove widget '%" INT64_FMT "' on the server."), Widget->GetWidgetId().GetKey());

			if (RootList.RemoveEntry(Widget))
			{
				FUIFrameworkModule::AuthorityDetachWidgetFromParent(Widget);
			}
		}
	}
}

void UUIFrameworkPlayerComponent::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	check(Widget);
	RootList.RemoveEntry(Widget);
}

FOnPendingReplicationProcessed& UUIFrameworkPlayerComponent::GetOnPendingReplicationProcessed()
{
	return OnPendingReplicationProcessed;
}

FUIFrameworkWidgetTree& UUIFrameworkPlayerComponent::GetWidgetTree()
{
	return WidgetTree;
}

FUIFrameworkWidgetOwner UUIFrameworkPlayerComponent::GetWidgetOwner() const
{
	FUIFrameworkWidgetOwner Owner;
	Owner.PlayerController = GetPlayerController();
	return Owner;
}

void UUIFrameworkPlayerComponent::HandleAddPending()
{
	ensureMsgf(AddPending.Num() != 0, TEXT("we shouldn't call HandleAddPending when nothing needs to be added"));
	// Create and add all the pending widgets
	TGuardValue<bool> TmpGuard = { bAddingWidget, true };
	for (int32 ReplicationId : AddPending)
	{
		if (FUIFrameworkWidgetTreeEntry* Entry = WidgetTree.LocalGetEntryByReplicationId(ReplicationId))
		{
			UE_LOG(LogUIFramework, VeryVerbose, TEXT("Attempting to add pending widget '%" INT64_FMT "'."), Entry->ChildId.GetKey());
			if (ensure(Entry->IsParentValid() && Entry->IsChildValid()))
			{
				if (Entry->ParentId.IsRoot())
				{
					LocalAddChild(Entry->ChildId);
				}
				else
				{
					Entry->Parent->LocalAddChild(Entry->ChildId);
				}
			}
		}
		else
		{
			UE_LOGF(LogUIFramework, Verbose, "A widget would have been added but couldn't be found anymore.");
		}
	}

	ensureMsgf(NetReplicationPending.Num() == 0, TEXT("we shouldn't be in HandleAddPending if we're waiting on net replication for some of the objects"));
	ensureMsgf(ClassesToLoad.Num() == 0, TEXT("we shouldn't be in HandleAddPending if our classes haven't been loaded yet"));
	AddPending.Empty();

	OnPendingReplicationProcessed.Broadcast();

	PrimaryComponentTick.SetTickFunctionEnable(false);

	if (bIsPendingFocusCall && WidgetToFocus.Id.IsValid())
	{
		if (UUIFrameworkWidget* Widget = GetWidgetTree().FindWidgetById(WidgetToFocus.Id))
		{
			if (UWidget* UMGWidget = Widget->LocalGetUMGWidget())
			{
				if (!PendingWidgetConstructedTickerHandle.IsValid())
				{
					PendingWidgetConstructedTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UUIFrameworkPlayerComponent::TrySetFocus, WidgetToFocus.Id));
					bIsPendingFocusCall = false;
				}
			}
		}
	}

}

UUIFrameworkPresenter* UUIFrameworkPlayerComponent::GetOrCreatePresenter()
{
	if (Presenter == nullptr)
	{
		Presenter = NewObject<UUIFrameworkPresenter>(this, FUIFrameworkModule::GetPresenterClass());
	}

	check(Presenter);
	return Presenter;
}

void UUIFrameworkPlayerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	APlayerController* LocalOwner = GetPlayerController();
	check(!LocalOwner || !LocalOwner->HasAuthority());

	if (ClassesToLoad.Num() != 0 || NetReplicationPending.Num() != 0)
	{
		FMemMark Mark(FMemStack::Get());
		TArray<int32, TMemStackAllocator<>> TempNetReplicationPending;
		TempNetReplicationPending.Reserve(NetReplicationPending.Num());
		for (int32 ReplicationId : NetReplicationPending)
		{
			TempNetReplicationPending.Add(ReplicationId);
		}
		NetReplicationPending.Empty(false);

		for (int32 ReplicationId : TempNetReplicationPending)
		{
			if (const FUIFrameworkWidgetTreeEntry* Entry = WidgetTree.LocalGetEntryByReplicationId(ReplicationId))
			{
				LocalWidgetWasAddedToTree(*Entry);
			}
		}
	}

	if (AddPending.Num() > 0 && ClassesToLoad.Num() == 0 && NetReplicationPending.Num() == 0)
	{
		UE_LOGF(LogUIFramework, VeryVerbose, "Pending widgets found, attempting to add.");
		HandleAddPending();
	}

}

void UUIFrameworkPlayerComponent::LocalWidgetWasAddedToTree(const FUIFrameworkWidgetTreeEntry& Entry)
{
	APlayerController* LocalOwner = GetPlayerController();
	check(!LocalOwner || !LocalOwner->HasAuthority());

	if (Entry.Child && Entry.Child->LocalIsReplicationReady())
	{

		TSoftClassPtr<UWidget> WidgetClass = Entry.Child->GetUMGWidgetClass();
		if (WidgetClass.IsValid())
		{
			UE_LOG(LogUIFramework, VeryVerbose, TEXT("Widget %s with ID  '%" INT64_FMT "' is ready to be created on the client."), *WidgetClass->GetName(), Entry.ChildId.GetKey());
			if (Entry.IsParentValid() && Entry.IsChildValid())
			{
				bool bDidCreateWidget = false;
				Entry.Child->LocalCreateUMGWidget(this, &bDidCreateWidget);
				if (bDidCreateWidget)
				{
					AddPending.Add(Entry.ReplicationID);
				}
				NetReplicationPending.Remove(Entry.ReplicationID);
			}
			else
			{
				NetReplicationPending.Add(Entry.ReplicationID);
			}
		}
		else if (!WidgetClass.IsNull() && WidgetClass.IsPending())
		{
			UE_LOG(LogUIFramework, VeryVerbose, TEXT("The widget '%" INT64_FMT "' needs to load its WidgetClass soft pointer."), Entry.ChildId.GetKey());

			if (FWidgetClassToLoad* FoundWidgetClassToLoad = ClassesToLoad.Find(WidgetClass))
			{
				FoundWidgetClassToLoad->EntryReplicationIds.AddUnique(Entry.ReplicationID);
			}
			else
			{
				// The class needs to be loaded
				TWeakObjectPtr<ThisClass> WeakSelf = this;
				TSharedPtr<FStreamableHandle> StreamableHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
					WidgetClass.ToSoftObjectPath()
					, [WeakSelf, WidgetClass]() mutable
					{
						if (ThisClass* StrongSelf = WeakSelf.Get())
						{
							StrongSelf->LocalOnClassLoaded(WidgetClass);
						}
					}
				, FStreamableManager::AsyncLoadHighPriority, false, false, TEXT("UIWidget Widget Class"));

				FWidgetClassToLoad NewItem;
				NewItem.EntryReplicationIds.Add(Entry.ReplicationID);
				NewItem.StreamableHandle = MoveTemp(StreamableHandle);
				ClassesToLoad.Add(WidgetClass, MoveTemp(NewItem));
			}
		}
		else
		{
			ensureMsgf(false, TEXT("The widget '%s' doesn't have it's WidgetClass property set."), *Entry.Child->GetClass()->GetName());
		}
	}
	else
	{
		UE_LOG(LogUIFramework, VeryVerbose, TEXT("Widget '%" INT64_FMT "' is not ready for replication yet."), Entry.ChildId.GetKey());

		NetReplicationPending.Add(Entry.ReplicationID);
	}

	PrimaryComponentTick.SetTickFunctionEnable(NetReplicationPending.Num() > 0 || AddPending.Num() > 0 || ClassesToLoad.Num() > 0);
}

void UUIFrameworkPlayerComponent::LocalWidgetRemovedFromTree(const FUIFrameworkWidgetTreeEntry& Entry)
{
	APlayerController* LocalOwner = GetPlayerController();
	check(!LocalOwner || !LocalOwner->HasAuthority());
	check(!bAddingWidget);

	NetReplicationPending.Remove(Entry.ReplicationID);
	AddPending.Remove(Entry.ReplicationID);
	UE_LOG(LogUIFramework, VeryVerbose, TEXT("Removed Widget '%" INT64_FMT "' from the widget tree locally."), Entry.ChildId.GetKey());

	PrimaryComponentTick.SetTickFunctionEnable(NetReplicationPending.Num() > 0 || AddPending.Num() > 0 || ClassesToLoad.Num() > 0);
}

void UUIFrameworkPlayerComponent::LocalRemoveWidgetRootFromTree(const UUIFrameworkWidget* Widget)
{
	check(Widget);
	ServerRemoveWidgetRootFromTree(Widget->GetWidgetId());
}

void UUIFrameworkPlayerComponent::ServerRemoveWidgetRootFromTree_Implementation(FUIFrameworkWidgetId WidgetId)
{
	if (UUIFrameworkWidget* Widget = GetWidgetTree().FindWidgetById(WidgetId))
	{
		GetWidgetTree().AuthorityRemoveWidgetAndChildren(Widget);
	}
}

void UUIFrameworkPlayerComponent::SetWidgetToFocus(FUIFrameworkWidgetId WidgetId)
{
	WidgetToFocus.Id = WidgetId;
	WidgetToFocus.ReplicationCounter++;
	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, WidgetToFocus, this);
}

void UUIFrameworkPlayerComponent::LocalOnClassLoaded(TSoftClassPtr<UWidget> WidgetClass)
{
	FWidgetClassToLoad FoundWidgetClassToLoad;
	if (ClassesToLoad.RemoveAndCopyValue(WidgetClass, FoundWidgetClassToLoad))
	{
		if (WidgetClass.Get())
		{
			for (int32 ReplicationId : FoundWidgetClassToLoad.EntryReplicationIds)
			{
				if (const FUIFrameworkWidgetTreeEntry* Entry = WidgetTree.LocalGetEntryByReplicationId(ReplicationId))
				{
					if (Entry->IsParentValid() && Entry->IsChildValid())
					{
						bool bDidCreateWidget = false;
						Entry->Child->LocalCreateUMGWidget(this, &bDidCreateWidget);
						if (bDidCreateWidget)
						{
							AddPending.Add(ReplicationId);
						}
						NetReplicationPending.Remove(ReplicationId);
					}
					else
					{
						NetReplicationPending.Add(ReplicationId);
					}
				}
				else
				{
					UE_LOGF(LogUIFramework, Log, "A widget with class %ls was removed.", *WidgetClass.Get()->GetName());
				}
			}
		}
		else
		{
			UE_LOGF(LogUIFramework, Error, "Failed to load widget class %ls.", *WidgetClass.ToString());
			ensure(false);
		}
	}
	else
	{
		UE_LOGF(LogUIFramework, Log, "A load request for class %ls was not found but could had be removed.", *WidgetClass.Get()->GetName());
	}

	PrimaryComponentTick.SetTickFunctionEnable(NetReplicationPending.Num() > 0 || AddPending.Num() > 0 || ClassesToLoad.Num() > 0);
}

void UUIFrameworkPlayerComponent::LocalAddChild(FUIFrameworkWidgetId WidgetId)
{
	UUIFrameworkWidget* Widget = GetWidgetTree().FindWidgetById(WidgetId);
	if (FUIFrameworkGameLayerSlot* LayerSlot = RootList.FindEntry(WidgetId))
	{
		if (Widget)
		{
			UWidget* UMGWidget = Widget->LocalGetUMGWidget();
			if (ensure(UMGWidget))
			{
				UMGWidget->RemoveFromParent();
				LayerSlot->LocalAquireWidget();
				if (UUIFrameworkPresenter* PresenterPtr = GetOrCreatePresenter())
				{
					PresenterPtr->AddToViewport(UMGWidget, *LayerSlot);
					UE_LOGF(LogUIFramework, VeryVerbose, "Widget %ls added to the local viewport via the presenter.", *UMGWidget->GetName());

				}
			}
		}
		else
		{
			UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), WidgetId.GetKey());
		}
	}
	else
	{
		if (Widget && Widget->LocalGetUMGWidget())
		{
			Widget->LocalGetUMGWidget()->RemoveFromParent();
		}
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the RootList."), WidgetId.GetKey());
	}
}

bool UUIFrameworkPlayerComponent::TrySetFocus(float DeltaTime, FUIFrameworkWidgetId CurrentWidgetToFocusId)
{
	if (UUIFrameworkWidget* Widget = GetWidgetTree().FindWidgetById(CurrentWidgetToFocusId))
	{
		if (UWidget* UMGWidget = Widget->LocalGetUMGWidget())
		{
			if (UMGWidget->IsConstructed())
			{
				const FUIFrameworkWidgetTreeEntry* RootEntry = GetWidgetTree().FindRootEntryById(CurrentWidgetToFocusId);
				if (RootEntry)
				{
					if (FUIFrameworkGameLayerSlot* LayerSlot = RootList.FindEntry(RootEntry->ChildId))
					{
						if (LayerSlot->InputMode == EUIFrameworkInputMode::UI)
						{
							if (UUIFrameworkPresenter* PresenterPtr = GetOrCreatePresenter())
							{
								PresenterPtr->FocusWidget(UMGWidget);
							}
						}
					}
				}
				FTSTicker::RemoveTicker(PendingWidgetConstructedTickerHandle);
			}
		}
	}
	return true;
}

void UUIFrameworkPlayerComponent::OnRep_WidgetToFocus()
{
	if (WidgetToFocus.Id.IsValid())
	{
		bIsPendingFocusCall = true;
		if (UUIFrameworkWidget* Widget = GetWidgetTree().FindWidgetById(WidgetToFocus.Id))
		{
			if (UWidget* UMGWidget = Widget->LocalGetUMGWidget())
			{
				if (UMGWidget->IsConstructed())
				{
					TrySetFocus(0, WidgetToFocus.Id);
					bIsPendingFocusCall = false;
					return;
				}
			}
			if (PendingWidgetConstructedTickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(PendingWidgetConstructedTickerHandle);
			}
			PendingWidgetConstructedTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UUIFrameworkPlayerComponent::TrySetFocus, WidgetToFocus.Id));
			bIsPendingFocusCall = false;
		}
	}
}