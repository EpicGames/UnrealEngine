// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFPresenter.h"

#include "Blueprint/GameViewportSubsystem.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFPresenter)

UUIFrameworkGameViewportPresenter::FWidgetPair::FWidgetPair(UWidget* InWidget, FUIFrameworkWidgetId InWidgetId)
	: UMGWidget(InWidget)
	, WidgetId(InWidgetId)
{

}

void UUIFrameworkGameViewportPresenter::AddToViewport(UWidget* UMGWidget, const FUIFrameworkGameLayerSlot& Slot)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		FGameViewportWidgetSlot GameViewportWidgetSlot;
		GameViewportWidgetSlot.ZOrder = Slot.ZOrder;
		if (Slot.Type == EUIFrameworkGameLayerType::Viewport)
		{
			Subsystem->AddWidget(UMGWidget, GameViewportWidgetSlot);
		}
		else
		{
			APlayerController* LocalOwner = GetOuterUUIFrameworkPlayerComponent()->GetPlayerController();
			check(LocalOwner);
			Subsystem->AddWidgetForPlayer(UMGWidget, LocalOwner->GetLocalPlayer(), GameViewportWidgetSlot);
		}
		Widgets.Emplace(UMGWidget, Slot.GetWidgetId());
	}
}

void UUIFrameworkGameViewportPresenter::RemoveFromViewport(FUIFrameworkWidgetId WidgetId)
{
	int32 IndexOf = Widgets.IndexOfByPredicate([WidgetId](const FWidgetPair& Other) { return Other.WidgetId == WidgetId; });
	if (IndexOf != INDEX_NONE)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			Subsystem->RemoveWidget(Widgets[IndexOf].UMGWidget.Get());
		}

		Widgets.RemoveAtSwap(IndexOf);
	}
}

void UUIFrameworkGameViewportPresenter::FocusWidget(UWidget* UMGWidget)
{
	if (UMGWidget)
	{
		UMGWidget->SetFocus();
	}
}

void UUIFrameworkGameViewportPresenter::BeginDestroy()
{
	if (GetOuter() && !IsTemplate())
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			for (FWidgetPair& Pair : Widgets)
			{
				Subsystem->RemoveWidget(Pair.UMGWidget.Get());
			}
		}
	}
	Widgets.Reset();

	Super::BeginDestroy();
}
