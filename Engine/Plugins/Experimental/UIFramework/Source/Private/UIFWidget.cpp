// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFWidget.h"
//#include "UIFManagerSubsystem.h"

#include "Blueprint/UserWidget.h"
#include "Engine/AssetManager.h"

#include "Engine/Engine.h"
#include "Engine/NetDriver.h"
#include "Net/UnrealNetwork.h"
#include "Templates/NonNullPointer.h"
#include "Types/UIFWidgetTree.h"
#include "Types/UIFWidgetOwner.h"
#include "Types/UIFWidgetTreeOwner.h"
#include "UIFLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFWidget)

UUIFrameworkWidget::UUIFrameworkWidget()
: Super()
{
	if (!IsTemplate())
	{
		Id = FUIFrameworkWidgetId::MakeNew();
	}
}

UUIFrameworkWidget::UUIFrameworkWidget(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	if (!IsTemplate())
	{
		Id = FUIFrameworkWidgetId::MakeNew();
	}
}

void UUIFrameworkWidget::ForceNetUpdate()
{
	if (AActor* OwnerActor = Cast<AActor>(GetOuter()))
	{
		OwnerActor->ForceNetUpdate();
	}
}

/**
 *
 */
int32 UUIFrameworkWidget::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	if (HasAnyFlags(RF_ClassDefaultObject) || !IsSupportedForNetworking())
	{
		// This handles absorbing authority/cosmetic
		return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
	}
	if (AActor* OwnerActor = Cast<AActor>(GetOuter()))
	{
		return OwnerActor->GetFunctionCallspace(Function, Stack);
	}
	return Super::GetFunctionCallspace(Function, Stack);
}

bool UUIFrameworkWidget::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	check(!HasAnyFlags(RF_ClassDefaultObject));

	bool bProcessed = false;
	AActor* OwnerActor = Cast<AActor>(GetOuter());
	FWorldContext* const Context = OwnerActor ? GEngine->GetWorldContextFromWorld(OwnerActor->GetWorld()) : nullptr;
	if (Context)
	{
		for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver && Driver.NetDriver->ShouldReplicateFunction(OwnerActor, Function))
			{
				Driver.NetDriver->ProcessRemoteFunction(OwnerActor, Function, Parameters, OutParms, Stack, this);
				bProcessed = true;
			}
		}
	}

	return bProcessed;
}

void UUIFrameworkWidget::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Id, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, bIsEnabled, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Visibility, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, RenderOpacity, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, WidgetClass, Params);
} 

FUIFrameworkWidgetTree* UUIFrameworkWidget::GetWidgetTree() const
{
	return WidgetTreeOwner ? &WidgetTreeOwner->GetWidgetTree() : nullptr;
}

void UUIFrameworkWidget::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	// By default we should remove the widget from its previous parent.
	//Adding a widget to a new slot will automatically remove it from its previous parent.
	if (FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree())
	{
		if (UUIFrameworkWidget* Widget = WidgetTree->FindWidgetById(ChildId))
		{
			if (UWidget* UMGWidget = Widget->LocalGetUMGWidget())
			{
				UMGWidget->RemoveFromParent();
			}
		}
	}
}

UWidget* UUIFrameworkWidget::LocalGetOrCreateUMGWidgetIfReady()
{
	if (UWidget* Widget = LocalGetUMGWidget())
	{
		if (Widget->GetClass() == WidgetClass.Get())
		{
			return Widget;
		}
	}
	
	if (!LocalIsReplicationReady())
	{
		return nullptr;
	}
	
	if (WidgetClass.IsNull())
	{
		return nullptr;
	}

	if (WidgetClass.IsPending())
	{
		AsyncLoadWidgetClass();
		return nullptr;
	}

	LocalCreateUMGWidget(nullptr, nullptr);
	return LocalGetUMGWidget();
}

TSharedPtr<FStreamableHandle> UUIFrameworkWidget::AsyncLoadWidgetClass()
{
	if (WidgetClassStreamableHandle.IsValid())
	{
		// Check that we're loading the correct class
		TArray<FSoftObjectPath> LoadingWidgets;
		constexpr bool bIncludeChildHandles = false;
		WidgetClassStreamableHandle->GetRequestedAssets(LoadingWidgets, bIncludeChildHandles);
		if (LoadingWidgets.Contains(WidgetClass.ToSoftObjectPath()))
		{
			return WidgetClassStreamableHandle;
		}

		// WidgetClass changed while we were loading
		WidgetClassStreamableHandle->CancelHandle();
		WidgetClassStreamableHandle.Reset();
	}
	
	TWeakObjectPtr<ThisClass> WeakSelf = this;
	WidgetClassStreamableHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
		WidgetClass.ToSoftObjectPath()
		, [WeakSelf]()
		{
			if (ThisClass* StrongSelf = WeakSelf.Get())
			{
				StrongSelf->LocalCreateUMGWidget(nullptr, nullptr);
				StrongSelf->WidgetClassStreamableHandle.Reset();
			}
		}
	, FStreamableManager::AsyncLoadHighPriority, false, false, TEXT("UUIFrameworkWidget::WidgetClass"));

	return WidgetClassStreamableHandle;
}

void UUIFrameworkWidget::LocalCreateUMGWidget(IUIFrameworkWidgetTreeOwner* InOwner, bool* bDidCreateWidgetPtr)
{
	bool bIsNewWidget = false;
	WidgetTreeOwner = InOwner;
	if (UClass* Class = WidgetClass.Get())
	{
		if (LocalUMGWidget && LocalUMGWidget->GetClass() != Class)
		{
			LocalUMGWidget->RemoveFromParent();
			LocalUMGWidget = nullptr;
		}

		if (!LocalUMGWidget)
		{
			if (Class->IsChildOf(UUserWidget::StaticClass()))
			{
				FUIFrameworkWidgetOwner UserWidgetOwner = WidgetTreeOwner ? WidgetTreeOwner->GetWidgetOwner() : FUIFrameworkWidgetOwner{ GetPlayerController() };
				if (UserWidgetOwner.PlayerController)
				{
					LocalUMGWidget = CreateWidget(UserWidgetOwner.PlayerController, Class);
				}
				else if (UserWidgetOwner.GameInstance)
				{
					LocalUMGWidget = CreateWidget(UserWidgetOwner.GameInstance, Class);
				}
				else if (UserWidgetOwner.World)
				{
					LocalUMGWidget = CreateWidget(UserWidgetOwner.World, Class);
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("There are no valid UserWidget owner."));
				}
			}
			else
			{
				check(Class->IsChildOf(UWidget::StaticClass()));
				LocalUMGWidget = NewObject<UWidget>(this, Class, FName(), RF_Transient);
			}
			
			bIsNewWidget = LocalUMGWidget != nullptr;
		}

		if (LocalUMGWidget != nullptr)
		{
			LocalUMGWidget->SetIsEnabled(bIsEnabled);
			LocalUMGWidget->SetVisibility(Visibility);

			if (bIsNewWidget)
			{
				LocalOnUMGWidgetCreated();
			}
		}

		if (LocalUMGWidget)
		{
			UE_LOGF(LogUIFramework, VeryVerbose, "UMG Widget object %ls created on the client.", *LocalUMGWidget->GetName());

		}
		else
		{ 
			UE_LOG(LogUIFramework, VeryVerbose, TEXT("No UMG Widget object was created for widget ID  '%" INT64_FMT "'."), Id.GetKey());
		}

	}

	if (bDidCreateWidgetPtr)
	{
		*bDidCreateWidgetPtr = bIsNewWidget;
	}
}

void UUIFrameworkWidget::LocalDestroyUMGWidget()
{
	if (LocalUMGWidget)
	{
		LocalUMGWidget->RemoveFromParent();
		LocalUMGWidget->ReleaseSlateResources(true);
	}
	LocalUMGWidget = nullptr;
	WidgetTreeOwner = nullptr;
}


ESlateVisibility UUIFrameworkWidget::GetVisibility() const
{
	return Visibility;
}

void UUIFrameworkWidget::SetVisibility(ESlateVisibility InVisibility)
{
	if (InVisibility == ESlateVisibility::Visible && !bIsHitTestVisible)
	{
		InVisibility = ESlateVisibility::SelfHitTestInvisible;
	}

	if (Visibility != InVisibility)
	{
		Visibility = InVisibility;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Visibility, this);
		ForceNetUpdate();
	}
}

bool UUIFrameworkWidget::IsEnabled() const
{
	return bIsEnabled;
}

void UUIFrameworkWidget::SetEnabled(bool bInIsEnabled)
{
	if (bIsEnabled != bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, bIsEnabled, this);
		ForceNetUpdate();
	}
}

bool UUIFrameworkWidget::IsHitTestVisible() const
{
	return bIsHitTestVisible;
}

void UUIFrameworkWidget::SetHitTestVisible(bool bInHitTestVisible)
{
	if (bIsHitTestVisible != bInHitTestVisible)
	{
		bIsHitTestVisible = bInHitTestVisible;

		if (Visibility == ESlateVisibility::Visible && !bIsHitTestVisible)
		{
			SetVisibility(ESlateVisibility::SelfHitTestInvisible);
		}
		else if (Visibility == ESlateVisibility::SelfHitTestInvisible && bIsHitTestVisible)
		{
			SetVisibility(ESlateVisibility::Visible);
		}
	}
}

double UUIFrameworkWidget::GetRenderOpacity() const
{
	return RenderOpacity;
}

void UUIFrameworkWidget::SetRenderOpacity(double InRenderOpacity)
{
	if (RenderOpacity != InRenderOpacity)
	{
		RenderOpacity = InRenderOpacity;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, RenderOpacity, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkWidget::OnRep_IsEnabled()
{
	if (LocalUMGWidget)
	{
		LocalUMGWidget->SetIsEnabled(bIsEnabled);
	}
}

void UUIFrameworkWidget::OnRep_Visibility()
{
	if (LocalUMGWidget)
	{
		LocalUMGWidget->SetVisibility(Visibility);
	}
}

void UUIFrameworkWidget::OnRep_RenderOpacity()
{
	if (LocalUMGWidget)
	{
		LocalUMGWidget->SetRenderOpacity((float)RenderOpacity);
	}
}
