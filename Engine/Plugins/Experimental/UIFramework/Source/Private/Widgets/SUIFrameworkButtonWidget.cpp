// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUIFrameworkButtonWidget.h"

SLATE_IMPLEMENT_WIDGET(SUIFrameworkButtonWidget)

void SUIFrameworkButtonWidget::Construct(const FArguments& InArgs)
{
	OnTouchLongPressedDelegate = InArgs._OnTouchLongPressedDelegate;
	OnTouchStartedDelegate = InArgs._OnTouchStartedDelegate;
	OnTouchEndedDelegate = InArgs._OnTouchEndedDelegate;

	SButton::Construct(SButton::FArguments()
		.ButtonStyle(InArgs._ButtonStyle)
		.OnSlateButtonDragDetected(InArgs._OnSlateButtonDragDetected)
		.OnSlateButtonDragOver(InArgs._OnSlateButtonDragOver)
		.OnSlateButtonDragEnter(InArgs._OnSlateButtonDragEnter)
		.OnSlateButtonDragLeave(InArgs._OnSlateButtonDragLeave)
		.OnSlateButtonDrop(InArgs._OnSlateButtonDrop)
		.ClickMethod(InArgs._ClickMethod)
		.TouchMethod(InArgs._TouchMethod)
		.PressMethod(InArgs._PressMethod)
		.OnClicked(InArgs._OnClicked)
		.OnClicked(InArgs._OnClicked)
		.OnPressed(InArgs._OnPressed)
		.OnReleased(InArgs._OnReleased)
		.IsFocusable(InArgs._IsFocusable)
		.OnReceivedFocus(InArgs._OnReceivedFocus)
		.OnLostFocus(InArgs._OnLostFocus)
		.OnHovered(InArgs._OnHovered)
		.OnUnhovered(InArgs._OnUnhovered)
		.AllowDragDrop(InArgs._AllowDragDrop));
}

FReply SUIFrameworkButtonWidget::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent)
{
	if (GestureEvent.GetGestureType() == EGestureEvent::LongPress)
	{
		if (OnTouchLongPressedDelegate.IsBound())
		{
			return OnTouchLongPressedDelegate.Execute(GestureEvent);
		}
	}
	return FReply::Unhandled();
}

FReply SUIFrameworkButtonWidget::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsTouchEvent())
	{
		if (OnTouchStartedDelegate.IsBound())
		{
			return OnTouchStartedDelegate.Execute(MouseEvent);
		}
	}
	return FReply::Unhandled();
}

FReply SUIFrameworkButtonWidget::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (OnTouchEndedDelegate.IsBound())
	{
		return OnTouchEndedDelegate.Execute(InTouchEvent);
	}
	return FReply::Unhandled();
}