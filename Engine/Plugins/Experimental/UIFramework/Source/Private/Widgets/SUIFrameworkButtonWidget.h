// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

class FUIFInputPreProcessor;

DECLARE_DELEGATE_RetVal_OneParam(
	FReply,
	FOnTouch,
	const FPointerEvent&)

/**
 * Slate button for UUIFrameworkButtonWidget. It exposes touch events for start, end, and long press.
 */
class SUIFrameworkButtonWidget : public SButton
{

public:
	SLATE_BEGIN_ARGS(SUIFrameworkButtonWidget)
		: _OnTouchLongPressedDelegate()
		, _OnTouchStartedDelegate()
		, _OnTouchEndedDelegate()
		, _ButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FButtonStyle >("Button"))
		, _OnSlateButtonDragDetected()
		, _OnSlateButtonDragEnter()
		, _OnSlateButtonDragLeave()
		, _OnSlateButtonDragOver()
		, _OnSlateButtonDrop()
		, _ClickMethod(EButtonClickMethod::DownAndUp)
		, _TouchMethod(EButtonTouchMethod::DownAndUp)
		, _PressMethod(EButtonPressMethod::DownAndUp)
		, _IsFocusable(true)
		, _AllowDragDrop(false)
		{
		}

		SLATE_EVENT(FOnTouch, OnTouchLongPressedDelegate)
		SLATE_EVENT(FOnTouch, OnTouchStartedDelegate)
		SLATE_EVENT(FOnTouch, OnTouchEndedDelegate)

		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		/** The visual style of the button */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		SLATE_EVENT(FOnClicked, OnClicked)

		SLATE_EVENT(FSimpleDelegate, OnPressed)

		SLATE_EVENT(FSimpleDelegate, OnReleased)

		SLATE_EVENT(FSimpleDelegate, OnHovered)

		SLATE_EVENT(FSimpleDelegate, OnUnhovered)

		SLATE_EVENT(FSimpleDelegate, OnReceivedFocus)

		SLATE_EVENT(FSimpleDelegate, OnLostFocus)

		// Drag and Drop
		SLATE_EVENT(FOnDragDetected, OnSlateButtonDragDetected)
		SLATE_EVENT(FOnDragEnter, OnSlateButtonDragEnter)
		SLATE_EVENT(FOnDragLeave, OnSlateButtonDragLeave)
		SLATE_EVENT(FOnDragOver, OnSlateButtonDragOver)
		SLATE_EVENT(FOnDrop, OnSlateButtonDrop)

		SLATE_ARGUMENT(EButtonClickMethod::Type, ClickMethod)
		SLATE_ARGUMENT(EButtonTouchMethod::Type, TouchMethod)
		SLATE_ARGUMENT(EButtonPressMethod::Type, PressMethod)

		SLATE_ARGUMENT(bool, IsFocusable)
		SLATE_ARGUMENT(bool, AllowDragDrop)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget overrides
	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent) override;
	virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	//~ SWidget overrides

private:
	FOnTouch OnTouchLongPressedDelegate;
	FOnTouch OnTouchStartedDelegate;
	FOnTouch OnTouchEndedDelegate;
};

