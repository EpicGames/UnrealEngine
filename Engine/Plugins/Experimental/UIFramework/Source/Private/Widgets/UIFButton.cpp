// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFButton.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"
#include "UIFrameworkButtonWidget.h"

#include "CommonInputSubsystem.h"
#include "Components/Button.h"
#include "Components/ButtonSlot.h"
#include "Engine/AssetManager.h"
#include "Misc/RedirectCollector.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFButton)

UUIFrameworkButton::UUIFrameworkButton()
{
// TODO: FIXME: Loading this asset fails. Temporary fixed by using UUIFrameworkButtonWidget::StaticClass()
// 	            Does this actually need to be an asset meant to be customizable or can it be set like above?

//	WidgetClass = FSoftObjectPath(TEXT("/UIFramework/Widgets/WBP_UIF_Button.WBP_UIF_Button"));
//#if WITH_EDITOR
//	// We need to ensure that this asset is cooked
//	GRedirectCollector.OnSoftObjectPathLoaded(WidgetClass.ToSoftObjectPath(), nullptr);
//#endif

	WidgetClass = UUIFrameworkButtonWidget::StaticClass();
}

void UUIFrameworkButton::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Slot, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, TriggeringInputAction, Params);
}

void UUIFrameworkButton::SetContent(FUIFrameworkSimpleSlot InEntry)
{
	bool bWidgetIsDifferent = Slot.AuthorityGetWidget() != InEntry.AuthorityGetWidget();
	if (bWidgetIsDifferent)
	{
		// Remove previous widget
		if (Slot.AuthorityGetWidget())
		{
			FUIFrameworkModule::AuthorityDetachWidgetFromParent(Slot.AuthorityGetWidget());
		}
	}

	Slot = InEntry;

	if (bWidgetIsDifferent && Slot.AuthorityGetWidget())
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		Slot.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, Slot.AuthorityGetWidget()));
	}

	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Slot, this);
	ForceNetUpdate();
}


void UUIFrameworkButton::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);
	if (UUIFrameworkWidget* ChildWidget = Slot.AuthorityGetWidget())
	{
		Func(ChildWidget);
	}
}

void UUIFrameworkButton::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);
	ensure(Widget == Slot.AuthorityGetWidget());

	Slot.AuthoritySetWidget(nullptr);;
	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Slot, this);
	ForceNetUpdate();
}

void UUIFrameworkButton::LocalOnUMGWidgetCreated()
{
	Super::LocalOnUMGWidgetCreated();
	UUIFrameworkButtonWidget* Button = CastChecked<UUIFrameworkButtonWidget>(LocalGetUMGWidget());
	Button->OnTriggered.AddUniqueDynamic(this, &ThisClass::HandleClick);
	Button->OnHovered.AddUniqueDynamic(this, &ThisClass::HandleHover);
	Button->OnUnhovered.AddUniqueDynamic(this, &ThisClass::HandleUnhover);
	Button->OnTouchLongPress.BindUObject(this, &ThisClass::HandleHighlight);
	Button->OnTouchEnded.BindUObject(this, &ThisClass::HandleUnhighlight);
	LoadInputAction();
}

void UUIFrameworkButton::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree();
	if (ChildId == Slot.GetWidgetId() && WidgetTree)
	{
		if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
		{
			UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
			if (ensure(ChildUMGWidget))
			{
				Slot.LocalAquireWidget();

				UButton* Button = CastChecked<UButton>(LocalGetUMGWidget());
				Button->ClearChildren();
				UButtonSlot* ButtonSlot = CastChecked<UButtonSlot>(Button->AddChild(ChildUMGWidget));
				ButtonSlot->SetPadding(Slot.Padding);
				ButtonSlot->SetHorizontalAlignment(Slot.HorizontalAlignment);
				ButtonSlot->SetVerticalAlignment(Slot.VerticalAlignment);
			}
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the Button Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}

void UUIFrameworkButton::SetTriggeringInputAction(TSoftObjectPtr<UInputAction> InputAction)
{
	if (InputAction != TriggeringInputAction)
	{
		TriggeringInputAction = InputAction;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, TriggeringInputAction, this);
	}
}

void UUIFrameworkButton::OnRep_TriggeringInputAction()
{
	LoadInputAction();
}

void UUIFrameworkButton::HandleInputActionLoaded()
{
#if !UE_SERVER
	if (UUIFrameworkButtonWidget* Button = Cast<UUIFrameworkButtonWidget>(LocalGetUMGWidget()))
	{
		Button->SetTriggeringEnhancedInputAction(TriggeringInputAction.Get());
	}
	AsyncLoadInputActionHandle.Reset();
#endif
}

void UUIFrameworkButton::LoadInputAction()
{
#if !UE_SERVER
	if (AsyncLoadInputActionHandle.IsValid())
	{
		AsyncLoadInputActionHandle->CancelHandle();
	}
	AsyncLoadInputActionHandle.Reset();

	if (!TriggeringInputAction.IsNull())
	{
		if (TriggeringInputAction.IsValid())
		{
			HandleInputActionLoaded();
		}
		else
		{
			FStreamableDelegate Delegate = FStreamableDelegate::CreateUObject(this, &UUIFrameworkButton::HandleInputActionLoaded);
			AsyncLoadInputActionHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(TriggeringInputAction.ToSoftObjectPath(), Delegate);
		}
	}
	else
	{
		HandleInputActionLoaded();
	}
#endif
}

void UUIFrameworkButton::HandleClick()
{
	// todo the click event should send the userid
	ServerClick(GetPlayerController());
}

void UUIFrameworkButton::ServerClick_Implementation(APlayerController* PlayerController)
{
	FUIFrameworkClickEventArgument Argument;
	Argument.PlayerController = PlayerController;
	Argument.Sender = this;
	OnClick.Broadcast(Argument);
}

void UUIFrameworkButton::HandleHover()
{
	if (const APlayerController* PlayerController = GetPlayerController())
	{
		if (const UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(PlayerController->GetLocalPlayer()))
		{
			if (CommonInputSubsystem->GetCurrentInputType() == ECommonInputType::Touch)
			{
				return;
			}
		}
	}

	HandleHighlight();
}

void UUIFrameworkButton::HandleUnhover()
{
	if (const APlayerController* PlayerController = GetPlayerController())
	{
		if (const UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(PlayerController->GetLocalPlayer()))
		{
			if (CommonInputSubsystem->GetCurrentInputType() == ECommonInputType::Touch)
			{
				return;
			}
		}
	}

	HandleUnhighlight();
}

void UUIFrameworkButton::HandleHighlight()
{
	if (!bIsHighlighted)
	{
		ServerHighlight(GetPlayerController());
		bIsHighlighted = true;
	}
}

void UUIFrameworkButton::HandleUnhighlight()
{
	if (bIsHighlighted)
	{
		ServerUnhighlight(GetPlayerController());
		bIsHighlighted = false;
	}
}

void UUIFrameworkButton::ServerHighlight_Implementation(APlayerController* PlayerController)
{
	FUIFrameworkHighlightEventArgument Argument;
	Argument.PlayerController = PlayerController;
	Argument.Sender = this;
	OnHighlight.Broadcast(Argument);
}

void UUIFrameworkButton::ServerUnhighlight_Implementation(APlayerController* PlayerController)
{
	FUIFrameworkUnhighlightEventArgument Argument;
	Argument.PlayerController = PlayerController;
	Argument.Sender = this;
	OnUnhighlight.Broadcast(Argument);
}

void UUIFrameworkButton::OnRep_Slot()
{
	if (LocalGetUMGWidget() && Slot.LocalIsAquiredWidgetValid())
	{
		if (UButtonSlot* ButtonSlot = Cast<UButtonSlot>(LocalGetUMGWidget()->Slot))
		{
			ButtonSlot->SetPadding(Slot.Padding);
			ButtonSlot->SetHorizontalAlignment(Slot.HorizontalAlignment);
			ButtonSlot->SetVerticalAlignment(Slot.VerticalAlignment);
		}
	}
	// else do not do anything, the slot was not added yet or it was modified but was not applied yet by the PlayerComponent
}
