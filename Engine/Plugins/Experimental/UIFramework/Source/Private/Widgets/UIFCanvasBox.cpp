// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFCanvasBox.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"

#include "Components/CanvasPanel.h"
#include "Components/CanvasPanelSlot.h"

#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFCanvasBox)


/**
 *
 */
void FUIFrameworkCanvasBoxSlotList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkCanvasBoxSlot& Slot = Slots[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			// Remove and add the widget again... 
			// that may not work if they are on top of each other... The order may matter if the zorder is the same :( 
			Owner->LocalAddChild(Slot.GetWidgetId());
		}
		//else it will be remove and the new widget will be added by the WidgetTree replication.
	}
}

bool FUIFrameworkCanvasBoxSlotList::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkCanvasBoxSlot, FUIFrameworkCanvasBoxSlotList>(Slots, DeltaParms, *this);
}

void FUIFrameworkCanvasBoxSlotList::AddEntry(FUIFrameworkCanvasBoxSlot Entry)
{
	FUIFrameworkCanvasBoxSlot& NewEntry = Slots.Add_GetRef(MoveTemp(Entry));
	MarkItemDirty(NewEntry);
}

bool FUIFrameworkCanvasBoxSlotList::RemoveEntry(UUIFrameworkWidget* Widget)
{
	check(Widget);
	const int32 Index = Slots.IndexOfByPredicate([Widget](const FUIFrameworkCanvasBoxSlot& Entry) { return Entry.AuthorityGetWidget() == Widget; });
	if (Index != INDEX_NONE)
	{
		Slots.RemoveAt(Index);
		MarkArrayDirty();
	}
	return Index != INDEX_NONE;
}

FUIFrameworkCanvasBoxSlot* FUIFrameworkCanvasBoxSlotList::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return Slots.FindByPredicate([WidgetId](const FUIFrameworkCanvasBoxSlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}

void FUIFrameworkCanvasBoxSlotList::ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	for (FUIFrameworkCanvasBoxSlot& Slot : Slots)
	{
		if (UUIFrameworkWidget* ChildWidget = Slot.AuthorityGetWidget())
		{
			Func(ChildWidget);
		}
	}
}

/**
 *
 */
UUIFrameworkCanvasBox::UUIFrameworkCanvasBox()
	: ReplicatedSlotList(this)
{
	WidgetClass = UCanvasPanel::StaticClass();
}

void UUIFrameworkCanvasBox::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ReplicatedSlotList, Params);
}

void UUIFrameworkCanvasBox::AddWidget(FUIFrameworkCanvasBoxSlot InEntry)
{
	if (InEntry.AuthorityGetWidget() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToAdd");
	}
	else
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		InEntry.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, InEntry.AuthorityGetWidget()));
		AddEntry(InEntry);
	}
}

void UUIFrameworkCanvasBox::RemoveWidget(UUIFrameworkWidget* Widget)
{
	if (Widget == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetToRemove");
	}
	else if (!Widget->AuthorityGetParent().IsParentValid())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetParentToRemoveFrom");
	}
	else if (!Widget->AuthorityGetParent().IsWidget())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is not a widget. Widget can't be removed."), ELogVerbosity::Warning, "NotWidgetParentToRemoveFrom");
	}
	else if (Widget->AuthorityGetParent().AsWidget() != this)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget was added to another widget parent. It can't be removed on this player."), ELogVerbosity::Warning, "InvalidPlayerParentOnRemovedWidget");
	}
	else
	{
		FUIFrameworkModule::AuthorityDetachWidgetFromParent(Widget);
		RemoveEntry(Widget);
	}
}

void UUIFrameworkCanvasBox::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);

	ReplicatedSlotList.ForEachChildren(Func);
}

void UUIFrameworkCanvasBox::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);
	RemoveEntry(Widget);
}

void UUIFrameworkCanvasBox::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	bool bIsAdded = false;
	if (FUIFrameworkCanvasBoxSlot* CanvasEntry = FindEntry(ChildId))
	{
		if (FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree())
		{
			if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
			{
				UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
				if (ensure(ChildUMGWidget))
				{
					UCanvasPanelSlot* CanvasSlot = CastChecked<UCanvasPanel>(LocalGetUMGWidget())->AddChildToCanvas(ChildUMGWidget);
					checkf(CanvasSlot, TEXT("CanvasPanel should be able to receive slot"));

					CanvasEntry->LocalAquireWidget();
					{
						FAnchorData AnchorData;
						AnchorData.Anchors = CanvasEntry->Anchors;
						AnchorData.Offsets = CanvasEntry->Offsets;
						AnchorData.Alignment = CanvasEntry->Alignment;
						CanvasSlot->SetLayout(AnchorData);
					}
					CanvasSlot->SetZOrder(CanvasEntry->ZOrder);
					CanvasSlot->SetAutoSize(CanvasEntry->bSizeToContent);
				}
			}
			else
			{
				UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
				Super::LocalAddChild(ChildId);
			}
		}
		else
		{
			UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
			Super::LocalAddChild(ChildId);
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the Canvas Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}

void UUIFrameworkCanvasBox::AddEntry(FUIFrameworkCanvasBoxSlot Entry)
{
	ReplicatedSlotList.AddEntry(Entry);
}

bool UUIFrameworkCanvasBox::RemoveEntry(UUIFrameworkWidget* Widget)
{
	return ReplicatedSlotList.RemoveEntry(Widget);
}

FUIFrameworkCanvasBoxSlot* UUIFrameworkCanvasBox::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return ReplicatedSlotList.FindEntry(WidgetId);
}
