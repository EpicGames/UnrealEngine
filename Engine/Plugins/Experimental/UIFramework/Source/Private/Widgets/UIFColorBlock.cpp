// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFColorBlock.h"

#include "Components/Image.h"

#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFColorBlock)

/**
 *
 */
UUIFrameworkColorBlock::UUIFrameworkColorBlock()
{
	WidgetClass = UImage::StaticClass();
}

void UUIFrameworkColorBlock::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Color, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DesiredSize, Params);
}

void UUIFrameworkColorBlock::SetColor(FLinearColor InColor)
{
	if (Color != InColor)
	{
		Color = InColor;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Color, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkColorBlock::SetDesiredSize(FVector2f InDesiredSize)
{
	if (DesiredSize != InDesiredSize)
	{
		DesiredSize = InDesiredSize;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, DesiredSize, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkColorBlock::LocalOnUMGWidgetCreated()
{
	UImage* Image = CastChecked<UImage>(LocalGetUMGWidget());

	FSlateBrush TmpBrush = Image->GetBrush();
	TmpBrush.ImageSize = FVector2D(DesiredSize);
	TmpBrush.TintColor = Color;

	Image->SetBrush(TmpBrush);
}

void UUIFrameworkColorBlock::OnRep_Color()
{
	if (LocalGetUMGWidget())
	{
		UUIFrameworkColorBlock::LocalOnUMGWidgetCreated();
	}
}

void UUIFrameworkColorBlock::OnRep_DesiredSize()
{
	if (LocalGetUMGWidget())
	{
		UUIFrameworkColorBlock::LocalOnUMGWidgetCreated();
	}
}

