// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFImageBlock.h"

#include "Components/Image.h"
#include "Engine/AssetManager.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "UIFLocalSettings.h"
#include "UIFLog.h"

#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFImageBlock)

/**
 *
 */
UUIFrameworkImageBlock::UUIFrameworkImageBlock()
{
	WidgetClass = UImage::StaticClass();
}

void UUIFrameworkImageBlock::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Data, Params);
}

void UUIFrameworkImageBlock::SetMaterial(TSoftObjectPtr<UMaterialInterface> InMaterial)
{
	if (Data.ResourceObject != InMaterial)
	{
		Data.ResourceObject = InMaterial;
		Data.bUseTextureSize = false;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Data, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkImageBlock::SetTexture(TSoftObjectPtr<UTexture2D> InTexture, bool bUseTextureSize)
{
	if (Data.ResourceObject != InTexture)
	{
		Data.ResourceObject = InTexture;
		Data.bUseTextureSize = bUseTextureSize;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Data, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkImageBlock::SetTint(FLinearColor InTint)
{
	if (Data.Tint != InTint)
	{
		Data.Tint = InTint;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Data, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkImageBlock::SetTiling(ESlateBrushTileType::Type InTiling)
{
	if (Data.Tiling != InTiling)
	{
		Data.Tiling = InTiling;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Data, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkImageBlock::SetDesiredSize(FVector2f InDesiredSize)
{
	if (Data.DesiredSize != InDesiredSize)
	{
		Data.DesiredSize = InDesiredSize;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Data, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkImageBlock::LocalOnUMGWidgetCreated()
{
	UImage* Image = CastChecked<UImage>(LocalGetUMGWidget());

	FSlateBrush TmpBrush = Image->GetBrush();
	TmpBrush.ImageSize = FVector2D(Data.DesiredSize);
	TmpBrush.TintColor = Data.Tint;
	TmpBrush.Tiling = Data.Tiling;

	UObject* Resource = ProcessResourceObject();
	if (Data.bUseTextureSize)
	{
		if (UTexture2D* Texture = Cast<UTexture2D>(Resource))
		{
			TmpBrush.ImageSize.X = static_cast<float>(Texture->GetSizeX());
			TmpBrush.ImageSize.Y = static_cast<float>(Texture->GetSizeY());
		}
	}
	TmpBrush.SetResourceObject(Resource);

	Image->SetBrush(TmpBrush);
}

bool UUIFrameworkImageBlock::LocalIsReplicationReady() const
{
	return Super::LocalIsReplicationReady() && (!bWaitForResourceToBeLoaded || Data.ResourceObject.IsValid() || Data.ResourceObject.IsNull());
}

void UUIFrameworkImageBlock::OnRep_Data()
{
	if (LocalGetUMGWidget())
	{
		UUIFrameworkImageBlock::LocalOnUMGWidgetCreated();
	}
	else if (bWaitForResourceToBeLoaded)
	{
		ProcessResourceObject();
	}
}

UObject* UUIFrameworkImageBlock::ProcessResourceObject()
{
	++StreamingCounter;
	if (StreamingHandle.IsValid())
	{
		StreamingHandle->CancelHandle();
		StreamingHandle.Reset();
	}

	if (Data.ResourceObject.IsNull())
	{
		return nullptr;
	}
	else if (Data.ResourceObject.IsPending())
	{
		TWeakObjectPtr<UUIFrameworkImageBlock> WeakThis(this);
		int32 CurrentStreamingCounter = StreamingCounter;
		StreamingHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
			Data.ResourceObject.ToSoftObjectPath(),
			[WeakThis, CurrentStreamingCounter]() {
				if (UUIFrameworkImageBlock* StrongThis = WeakThis.Get())
				{
					// If the StreamingCounter do not matches, then this delegate was interrupted, but had already been queued for a callback.
					if (StrongThis->StreamingCounter != CurrentStreamingCounter)
					{
						return;
					}

					StrongThis->OnRep_Data();
				}
			},
			FStreamableManager::AsyncLoadHighPriority);

		return GetDefault<UUIFrameworkLocalSettings>()->GetLoadingResource();
	}
	else
	{
		check(Data.ResourceObject.IsValid());
		UObject* ResourceObject = Data.ResourceObject.Get();
		UObject* Result = GetDefault<UUIFrameworkLocalSettings>()->GetErrorResource();
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(ResourceObject))
		{
			if (UMaterial* BaseMaterial = MaterialInterface->GetBaseMaterial())
			{
				if (BaseMaterial->IsUIMaterial())
				{
					Result = ResourceObject;
				}
				else
				{
					UE_LOGF(LogUIFramework, Error, "Can't use material %ls for material_block %ls. The material domain is not set to \"User Interface\"", *BaseMaterial->GetName(), *GetName());
				}
			}
		}
		else if (UTexture2D* Texture = Cast<UTexture2D>(ResourceObject))
		{
			Result = ResourceObject;
		}
		return Result;
	}
}
