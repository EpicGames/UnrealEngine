// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFOverlay.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"

#include "Components/Spacer.h"
#include "Components/Overlay.h"
#include "Components/OverlaySlot.h"

#include "Net/UnrealNetwork.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFOverlay)

/**
 *
 */
void FUIFrameworkOverlaySlotList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkOverlaySlot& Slot = Slots[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			Owner->LocalAddChild(Slot.GetWidgetId());
		}
	}
}

bool FUIFrameworkOverlaySlotList::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkOverlaySlot, FUIFrameworkOverlaySlotList>(Slots, DeltaParms, *this);
}

void FUIFrameworkOverlaySlotList::AddEntry(FUIFrameworkOverlaySlot Entry)
{
	int32 NewEntryIndex = Slots.Add(MoveTemp(Entry));
	FUIFrameworkOverlaySlot& NewEntry = Slots[NewEntryIndex];
	NewEntry.Index = NewEntryIndex;
	MarkItemDirty(NewEntry);
}

bool FUIFrameworkOverlaySlotList::RemoveEntry(UUIFrameworkWidget* Widget)
{
	check(Widget);
	int32 Index = Slots.IndexOfByPredicate([Widget](const FUIFrameworkOverlaySlot& Entry) { return Entry.AuthorityGetWidget() == Widget; });
	if (Index != INDEX_NONE)
	{
		Slots.RemoveAt(Index);
		for (; Index < Slots.Num(); ++Index)
		{
			Slots[Index].Index = Index;
		}
		MarkArrayDirty();
	}
	return Index != INDEX_NONE;
}

FUIFrameworkOverlaySlot* FUIFrameworkOverlaySlotList::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return Slots.FindByPredicate([WidgetId](const FUIFrameworkOverlaySlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}

void FUIFrameworkOverlaySlotList::ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	for (FUIFrameworkOverlaySlot& Slot : Slots)
	{
		Func(Slot.AuthorityGetWidget());
	}
}

/**
 *
 */
UUIFrameworkOverlay::UUIFrameworkOverlay()
	: ReplicatedSlotList(this)
{
	WidgetClass = UOverlay::StaticClass();
}

void UUIFrameworkOverlay::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ReplicatedSlotList, Params);
}

void UUIFrameworkOverlay::AddWidget(FUIFrameworkOverlaySlot InEntry)
{
	if (InEntry.AuthorityGetWidget() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToAdd");
	}
	else
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		InEntry.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, InEntry.AuthorityGetWidget()));
		AddEntry(MoveTemp(InEntry));
	}
}

void UUIFrameworkOverlay::RemoveWidget(UUIFrameworkWidget* Widget)
{
	if (Widget == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetToRemove");
	}
	else if (!Widget->AuthorityGetParent().IsParentValid())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetParentToRemoveFrom");
	}
	else if (!Widget->AuthorityGetParent().IsWidget())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is not a widget. Widget can't be removed."), ELogVerbosity::Warning, "NotWidgetParentToRemoveFrom");
	}
	else if (Widget->AuthorityGetParent().AsWidget() != this)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget was added to another widget parent. It can't be removed on this player."), ELogVerbosity::Warning, "InvalidPlayerParentOnRemovedWidget");
	}
	else
	{
		FUIFrameworkModule::AuthorityDetachWidgetFromParent(Widget);
		RemoveEntry(Widget);
	}
}

void UUIFrameworkOverlay::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);

	ReplicatedSlotList.ForEachChildren(Func);
}

void UUIFrameworkOverlay::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);
	RemoveEntry(Widget);
}

void UUIFrameworkOverlay::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	bool bIsAdded = false;
	if (FUIFrameworkOverlaySlot* OverlayEntry = FindEntry(ChildId))
	{
		if (FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree())
		{
			if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
			{
				UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
				if (ensure(ChildUMGWidget))
				{
					UOverlay* Overlay = CastChecked<UOverlay>(LocalGetUMGWidget());
					UPanelSlot* PanelSlot = nullptr;

					const int32 ChildCount = Overlay->GetChildrenCount();
					if (OverlayEntry->Index < ChildCount)
					{
						Overlay->ReplaceOverlayChildAt(OverlayEntry->Index, ChildUMGWidget);
						PanelSlot = Overlay->GetSlots()[OverlayEntry->Index];
					}
					else if (OverlayEntry->Index == ChildCount)
					{
						PanelSlot = Overlay->AddChild(ChildUMGWidget);
					}
					else
					{
						for (int32 Index = ChildCount; Index < OverlayEntry->Index; ++Index)
						{
							Overlay->AddChild(NewObject<USpacer>(GetTransientPackage()));
						}
						PanelSlot = Overlay->AddChild(ChildUMGWidget);
					}

					checkf(PanelSlot, TEXT("OverlayPanel should be able to receive slot"));
					
					OverlayEntry->LocalAquireWidget();

					UOverlaySlot* OverlaySlot = CastChecked<UOverlaySlot>(PanelSlot);
					OverlaySlot->SetPadding(OverlayEntry->Padding);
					OverlaySlot->SetVerticalAlignment(OverlayEntry->VerticalAlignment);
					OverlaySlot->SetHorizontalAlignment(OverlayEntry->HorizontalAlignment);
				}
			}
			else
			{
				UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
				Super::LocalAddChild(ChildId);
			}
		}
		else
		{
			UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
			Super::LocalAddChild(ChildId);
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the Overlay Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}

void UUIFrameworkOverlay::AddEntry(FUIFrameworkOverlaySlot Entry)
{
	ReplicatedSlotList.AddEntry(Entry);
}

bool UUIFrameworkOverlay::RemoveEntry(UUIFrameworkWidget* Widget)
{
	return ReplicatedSlotList.RemoveEntry(Widget);
}

FUIFrameworkOverlaySlot* UUIFrameworkOverlay::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return ReplicatedSlotList.FindEntry(WidgetId);
}
