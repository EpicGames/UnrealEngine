// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFSafeZoneBox.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"

#include "Components/SafeZone.h"
#include "Components/SafeZoneSlot.h"

#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFSafeZoneBox)


/**
 * 
 */
UUIFrameworkSafeZoneBox::UUIFrameworkSafeZoneBox()
{
	WidgetClass = USafeZone::StaticClass();
}

void UUIFrameworkSafeZoneBox::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Slot, Params);
}

void UUIFrameworkSafeZoneBox::SetContent(FUIFrameworkSlotBase InEntry)
{
	bool bWidgetIsDifferent = Slot.AuthorityGetWidget() != InEntry.AuthorityGetWidget();
	if (bWidgetIsDifferent)
	{
		// Remove previous widget
		if (Slot.AuthorityGetWidget())
		{
			FUIFrameworkModule::AuthorityDetachWidgetFromParent(Slot.AuthorityGetWidget());
		}
	}

	Slot = InEntry;

	if (bWidgetIsDifferent && Slot.AuthorityGetWidget())
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		Slot.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, Slot.AuthorityGetWidget()));
	}

	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Slot, this);
	ForceNetUpdate();
}


void UUIFrameworkSafeZoneBox::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);
	if (UUIFrameworkWidget* ChildWidget = Slot.AuthorityGetWidget())
	{
		Func(ChildWidget);
	}
}

void UUIFrameworkSafeZoneBox::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);
	ensure(Widget == Slot.AuthorityGetWidget());

	Slot.AuthoritySetWidget(nullptr);
	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Slot, this);
	ForceNetUpdate();
}


void UUIFrameworkSafeZoneBox::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree();
	if (ChildId == Slot.GetWidgetId() && WidgetTree)
	{
		if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
		{
			UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
			if (ensure(ChildUMGWidget))
			{
				Slot.LocalAquireWidget();

				USafeZone* SafeZone = CastChecked<USafeZone>(LocalGetUMGWidget());
				SafeZone->ClearChildren();
				USafeZoneSlot* SafeZoneSlot = CastChecked<USafeZoneSlot>(SafeZone->AddChild(ChildUMGWidget));
			}
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the Safe Zone Box Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}
