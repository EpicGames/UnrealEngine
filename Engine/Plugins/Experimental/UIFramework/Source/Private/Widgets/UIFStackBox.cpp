// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFStackBox.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"

#include "Components/Spacer.h"
#include "Components/StackBox.h"
#include "Components/StackBoxSlot.h"

#include "Net/UnrealNetwork.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFStackBox)


/**
 *
 */
void FUIFrameworkStackBoxSlotList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkStackBoxSlot& Slot = Slots[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			// Remove and add the widget again... 
			// that may not work if they are on top of each other... The order may matter if the zorder is the same :( 
			Owner->LocalAddChild(Slot.GetWidgetId());
		}
		//else it will be remove and the new widget will be added by the WidgetTree replication.
	}
}

bool FUIFrameworkStackBoxSlotList::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkStackBoxSlot, FUIFrameworkStackBoxSlotList>(Slots, DeltaParms, *this);
}

void FUIFrameworkStackBoxSlotList::AddEntry(FUIFrameworkStackBoxSlot Entry)
{
	int32 NewEntryIndex = Slots.Add(MoveTemp(Entry));
	FUIFrameworkStackBoxSlot& NewEntry = Slots[NewEntryIndex];
	NewEntry.Index = NewEntryIndex;
	MarkItemDirty(NewEntry);
}

bool FUIFrameworkStackBoxSlotList::RemoveEntry(UUIFrameworkWidget* Widget)
{
	check(Widget);
	int32 Index = Slots.IndexOfByPredicate([Widget](const FUIFrameworkStackBoxSlot& Entry) { return Entry.AuthorityGetWidget() == Widget; });
	if (Index != INDEX_NONE)
	{
		Slots.RemoveAt(Index);
		for (; Index < Slots.Num(); ++Index)
		{
			Slots[Index].Index = Index;
		}
		MarkArrayDirty();
	}
	return Index != INDEX_NONE;
}

FUIFrameworkStackBoxSlot* FUIFrameworkStackBoxSlotList::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return Slots.FindByPredicate([WidgetId](const FUIFrameworkStackBoxSlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}

void FUIFrameworkStackBoxSlotList::ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	for (FUIFrameworkStackBoxSlot& Slot : Slots)
	{
		Func(Slot.AuthorityGetWidget());
	}
}

/**
 *
 */
UUIFrameworkStackBox::UUIFrameworkStackBox()
	: ReplicatedSlotList(this)
{
	WidgetClass = UStackBox::StaticClass();
}

void UUIFrameworkStackBox::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Orientation, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ReplicatedSlotList, Params);
}

void UUIFrameworkStackBox::AddWidget(FUIFrameworkStackBoxSlot InEntry)
{
	if (InEntry.AuthorityGetWidget() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToAdd");
	}
	else
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		InEntry.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, InEntry.AuthorityGetWidget()));
		AddEntry(InEntry);
	}
}

void UUIFrameworkStackBox::RemoveWidget(UUIFrameworkWidget* Widget)
{
	if (Widget == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetToRemove");
	}
	else if (!Widget->AuthorityGetParent().IsParentValid())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is invalid. Widget can't be removed."), ELogVerbosity::Warning, "InvalidWidgetParentToRemoveFrom");
	}
	else if (!Widget->AuthorityGetParent().IsWidget())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget parent is not a widget. Widget can't be removed."), ELogVerbosity::Warning, "NotWidgetParentToRemoveFrom");
	}
	else if (Widget->AuthorityGetParent().AsWidget() != this)
	{
		FFrame::KismetExecutionMessage(TEXT("The widget was added to another widget parent. It can't be removed on this player."), ELogVerbosity::Warning, "InvalidPlayerParentOnRemovedWidget");
	}
	else
	{
		FUIFrameworkModule::AuthorityDetachWidgetFromParent(Widget);
		RemoveEntry(Widget);
	}
}

EOrientation UUIFrameworkStackBox::GetOrientation() const
{
	return Orientation;
}

void UUIFrameworkStackBox::SetOrientation(EOrientation Value)
{
	if (Orientation != Value)
	{
		Orientation = Value;
		if (UStackBox* StackBox = Cast<UStackBox>(LocalGetUMGWidget()))
		{
			StackBox->SetOrientation(Orientation);
		}
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Orientation, this);
		ForceNetUpdate();
	}
}


void UUIFrameworkStackBox::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);

	ReplicatedSlotList.ForEachChildren(Func);
}

void UUIFrameworkStackBox::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);
	RemoveEntry(Widget);
}

void UUIFrameworkStackBox::LocalOnUMGWidgetCreated()
{
	Super::LocalOnUMGWidgetCreated();
	UStackBox* StackBox = CastChecked<UStackBox>(LocalGetUMGWidget());
	StackBox->SetOrientation(Orientation);
}

void UUIFrameworkStackBox::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	bool bIsAdded = false;
	if (FUIFrameworkStackBoxSlot* StackBoxEntry = FindEntry(ChildId))
	{
		if (FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree())
		{
			if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
			{
				UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
				if (ensure(ChildUMGWidget))
				{
					UStackBox* StackBox = CastChecked<UStackBox>(LocalGetUMGWidget());
					UPanelSlot* PanelSlot = nullptr;
					{
						const int32 ChildCount = StackBox->GetChildrenCount();
						if (StackBoxEntry->Index < ChildCount)
						{
							// replace the existing slot
							StackBox->ReplaceStackBoxChildAt(StackBoxEntry->Index, ChildUMGWidget);
							PanelSlot = StackBox->GetSlots()[StackBoxEntry->Index];
						}
						else if (StackBoxEntry->Index == ChildCount)
						{
							// add to the end
							PanelSlot = StackBox->AddChild(ChildUMGWidget);
						}
						else
						{
							// insert until the widget count is correct
							for (int32 Index = ChildCount; Index < StackBoxEntry->Index; ++Index)
							{
								StackBox->AddChild(NewObject<USpacer>(GetTransientPackage()));
							}
							PanelSlot = StackBox->AddChild(ChildUMGWidget);
						}
						checkf(PanelSlot, TEXT("StackBoxPanel should be able to receive slot"));
					}

					StackBoxEntry->LocalAquireWidget();

					UStackBoxSlot* StackBoxSlot = CastChecked<UStackBoxSlot>(PanelSlot);
					StackBoxSlot->SetPadding(StackBoxEntry->Padding);
					StackBoxSlot->SetSize(StackBoxEntry->Size);
					StackBoxSlot->SetVerticalAlignment(StackBoxEntry->VerticalAlignment);
					StackBoxSlot->SetHorizontalAlignment(StackBoxEntry->HorizontalAlignment);
				}
			}
			else
			{
				UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
				Super::LocalAddChild(ChildId);
			}
		}
		else
		{
			UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
			Super::LocalAddChild(ChildId);
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the StackBox Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}

void UUIFrameworkStackBox::OnRep_Orientation()
{
	if (UStackBox* StackBox = Cast<UStackBox>(LocalGetUMGWidget()))
	{
		StackBox->SetOrientation(Orientation);
	}
}

void UUIFrameworkStackBox::AddEntry(FUIFrameworkStackBoxSlot Entry)
{
	ReplicatedSlotList.AddEntry(Entry);
}

bool UUIFrameworkStackBox::RemoveEntry(UUIFrameworkWidget* Widget)
{
	return ReplicatedSlotList.RemoveEntry(Widget);
}

FUIFrameworkStackBoxSlot* UUIFrameworkStackBox::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return ReplicatedSlotList.FindEntry(WidgetId);
}
