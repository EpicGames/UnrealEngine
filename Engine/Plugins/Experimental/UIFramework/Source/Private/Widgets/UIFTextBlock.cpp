// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFTextBlock.h"

#include "Components/TextBlock.h"
#include "ILocalizableMessageModule.h"
#include "Internationalization/Internationalization.h"
#include "LocalizableMessageProcessor.h"
#include "LocalizationContext.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFTextBlock)


/**
 *
 */
UUIFrameworkTextBase::UUIFrameworkTextBase()
{
	WidgetClass = nullptr;
}


void UUIFrameworkTextBase::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Message, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, TextColor, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, Justification, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, OverflowPolicy, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, AutoWrap, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, WrappingPolicy, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, WrapWidth, Params);
}

void UUIFrameworkTextBase::SetText(const FLocalizableMessage& InMessage)
{
	FLocalizationContext LocContext(this);
	ILocalizableMessageModule& LocalizableMessageModule = ILocalizableMessageModule::Get();
	FLocalizableMessageProcessor& Processor = LocalizableMessageModule.GetLocalizableMessageProcessor();
	Text = Processor.Localize(InMessage, LocContext);
}

void UUIFrameworkTextBase::SetMessage(FLocalizableMessage&& InMessage)
{
	if (Message != InMessage)
	{
		SetText(InMessage);

		Message = MoveTemp(InMessage);
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Message, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetTextColor(FLinearColor InTextColor)
{
	if (TextColor != InTextColor)
	{
		TextColor = InTextColor;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, TextColor, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetTextSize(float InTextSize)
{
	if (TextSize != InTextSize)
	{
		TextSize = InTextSize;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, TextSize, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetAutoWrap(bool InAutoWrap)
{
	if (AutoWrap != InAutoWrap)
	{
		AutoWrap = InAutoWrap;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, AutoWrap, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetWrapWidth(float InWrapWidth)
{
	if (WrapWidth != InWrapWidth)
	{
		WrapWidth = InWrapWidth;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, WrapWidth, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy)
{
	if (WrappingPolicy != InWrappingPolicy)
	{
		WrappingPolicy = InWrappingPolicy;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, WrappingPolicy, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetJustification(ETextJustify::Type InJustification)
{
	if (Justification != InJustification)
	{
		Justification = InJustification;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, Justification, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::SetOverflowPolicy(ETextOverflowPolicy InOverflowPolicy)
{
	if (OverflowPolicy != InOverflowPolicy)
	{
		OverflowPolicy = InOverflowPolicy;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, OverflowPolicy, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBase::LocalOnUMGWidgetCreated()
{
	SetText(Message);
	SetTextToWidget(Text);
	SetTextColorToWidget(TextColor);
	SetTextSizeToWidget(TextSize);
	SetAutoWrapToWidget(AutoWrap);
	SetWrapWidthToWidget(WrapWidth);
	SetWrappingPolicyToWidget(WrappingPolicy);
	SetJustificationToWidget(Justification);
	SetOverflowPolicyToWidget(OverflowPolicy);
}

void UUIFrameworkTextBase::OnRep_Message()
{
	SetText(Message);

	if (LocalGetUMGWidget())
	{
		SetTextToWidget(Text);
	}
}

void UUIFrameworkTextBase::OnRep_TextColor()
{
	if (LocalGetUMGWidget())
	{
		SetTextColorToWidget(TextColor);
	}
}

void UUIFrameworkTextBase::OnRep_TextSize()
{
	if (LocalGetUMGWidget())
	{
		SetTextSizeToWidget(TextSize);
	}
}

void UUIFrameworkTextBase::OnRep_AutoWrap()
{
	if (LocalGetUMGWidget())
	{
		SetAutoWrapToWidget(AutoWrap);
	}
}

void UUIFrameworkTextBase::OnRep_WrapWidth()
{
	if (LocalGetUMGWidget())
	{
		SetWrapWidthToWidget(WrapWidth);
	}
}

void UUIFrameworkTextBase::OnRep_WrappingPolicy()
{
	if (LocalGetUMGWidget())
	{
		SetWrappingPolicyToWidget(WrappingPolicy);
	}
}

void UUIFrameworkTextBase::OnRep_Justification()
{
	if (LocalGetUMGWidget())
	{
		SetJustificationToWidget(Justification);
	}
}

void UUIFrameworkTextBase::OnRep_OverflowPolicy()
{
	if (LocalGetUMGWidget())
	{
		SetOverflowPolicyToWidget(OverflowPolicy);
	}
}

/**
 * 
 */
UUIFrameworkTextBlock::UUIFrameworkTextBlock()
{
	WidgetClass = UTextBlock::StaticClass();
}

void UUIFrameworkTextBlock::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ShadowOffset, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ShadowColor, Params);
}

void UUIFrameworkTextBlock::SetShadowOffset(FVector2f InShadowOffset)
{
	if (ShadowOffset != InShadowOffset)
	{
		ShadowOffset = InShadowOffset;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, ShadowOffset, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBlock::SetShadowColor(FLinearColor InShadowColor)
{
	if (ShadowColor != InShadowColor)
	{
		ShadowColor = InShadowColor;
		MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, ShadowColor, this);
		ForceNetUpdate();
	}
}

void UUIFrameworkTextBlock::LocalOnUMGWidgetCreated()
{
	Super::LocalOnUMGWidgetCreated();
	SetShadowOffsetToWidget(ShadowOffset);
	SetShadowColorToWidget(ShadowColor);
}

void UUIFrameworkTextBlock::OnRep_ShadowOffset()
{
	if (LocalGetUMGWidget())
	{
		SetShadowOffsetToWidget(ShadowOffset);
	}
}

void UUIFrameworkTextBlock::OnRep_ShadowColor()
{
	if (LocalGetUMGWidget())
	{
		SetShadowColorToWidget(ShadowColor);
	}
}

void UUIFrameworkTextBlock::SetTextToWidget(const FText& InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetText(InValue);
}

void UUIFrameworkTextBlock::SetTextSizeToWidget(float InValue)
{
	UTextBlock* TextBlock = CastChecked<UTextBlock>(LocalGetUMGWidget());
	FSlateFontInfo CurrentFontInfo = TextBlock->GetFont();
	CurrentFontInfo.Size = InValue;
	TextBlock->SetFont(CurrentFontInfo);
}

void UUIFrameworkTextBlock::SetAutoWrapToWidget(bool InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetAutoWrapText(InValue);
}

void UUIFrameworkTextBlock::SetWrapWidthToWidget(float InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetWrapTextAt(InValue);
}

void UUIFrameworkTextBlock::SetWrappingPolicyToWidget(ETextWrappingPolicy InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetWrappingPolicy(InValue);
}

void UUIFrameworkTextBlock::SetTextColorToWidget(FLinearColor InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetColorAndOpacity(InValue);
}

void UUIFrameworkTextBlock::SetShadowOffsetToWidget(FVector2f InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetShadowOffset(FVector2d(InValue));
}

void UUIFrameworkTextBlock::SetShadowColorToWidget(FLinearColor InValue)
{
	CastChecked<UTextBlock>(LocalGetUMGWidget())->SetShadowColorAndOpacity(InValue);
}

void UUIFrameworkTextBlock::SetJustificationToWidget(ETextJustify::Type InJustification)
{
	if (LocalGetUMGWidget())
	{
		CastChecked<UTextBlock>(LocalGetUMGWidget())->SetJustification(InJustification);
	}
}

void UUIFrameworkTextBlock::SetOverflowPolicyToWidget(ETextOverflowPolicy InValue)
{
	if (LocalGetUMGWidget())
	{
		CastChecked<UTextBlock>(LocalGetUMGWidget())->SetTextOverflowPolicy(InValue);
	}
}
