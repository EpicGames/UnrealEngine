// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UIFUserWidget.h"
#include "Types/UIFWidgetTree.h"
#include "UIFLog.h"
#include "UIFModule.h"

#include "Blueprint/UserWidget.h"
#include "MVVMSubsystem.h"
#include "View/MVVMView.h"

#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFUserWidget)


/**
 *
 */
void FUIFrameworkUserWidgetNamedSlotList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	check(Owner);
	for (int32 Index : ChangedIndices)
	{
		FUIFrameworkUserWidgetNamedSlot& Slot = Slots[Index];
		if (Slot.LocalIsAquiredWidgetValid())
		{
			// Remove and add the widget again... 
			// that may not work if they are on top of each other... The order may matter if the zorder is the same :( 
			Owner->LocalAddChild(Slot.GetWidgetId());
		}
		//else it will be remove and the new widget will be added by the WidgetTree replication.
	}
}

bool FUIFrameworkUserWidgetNamedSlotList::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkUserWidgetNamedSlot, FUIFrameworkUserWidgetNamedSlotList>(Slots, DeltaParms, *this);
}

void FUIFrameworkUserWidgetNamedSlotList::AuthorityAddEntry(FUIFrameworkUserWidgetNamedSlot Entry)
{
	// Make sure there is only one entry of that name.
	int32 RemoveCount = Slots.RemoveAllSwap([SlotName = Entry.SlotName](const FUIFrameworkUserWidgetNamedSlot& Entry) { return Entry.SlotName == SlotName; });
	if (RemoveCount > 0)
	{
		MarkArrayDirty();
	}

	FUIFrameworkUserWidgetNamedSlot& NewEntry = Slots.Add_GetRef(MoveTemp(Entry));
	MarkItemDirty(NewEntry);
}

bool FUIFrameworkUserWidgetNamedSlotList::AuthorityRemoveEntry(UUIFrameworkWidget* Widget)
{
	check(Widget);
	const int32 Index = Slots.IndexOfByPredicate([Widget](const FUIFrameworkUserWidgetNamedSlot& Entry) { return Entry.AuthorityGetWidget() == Widget; });
	if (Index != INDEX_NONE)
	{
		Slots.RemoveAt(Index);
		MarkArrayDirty();
	}
	return Index != INDEX_NONE;
}

FUIFrameworkUserWidgetNamedSlot* FUIFrameworkUserWidgetNamedSlotList::FindEntry(FUIFrameworkWidgetId WidgetId)
{
	return Slots.FindByPredicate([WidgetId](const FUIFrameworkUserWidgetNamedSlot& Entry) { return Entry.GetWidgetId() == WidgetId; });
}

const FUIFrameworkUserWidgetNamedSlot* FUIFrameworkUserWidgetNamedSlotList::AuthorityFindEntry(FName SlotName) const
{
	return Slots.FindByPredicate([SlotName](const FUIFrameworkUserWidgetNamedSlot& Entry) { return Entry.SlotName == SlotName; });
}

void FUIFrameworkUserWidgetNamedSlotList::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	for (FUIFrameworkUserWidgetNamedSlot& Slot : Slots)
	{
		if (UUIFrameworkWidget* ChildWidget = Slot.AuthorityGetWidget())
		{
			Func(ChildWidget);
		}
	}
}

/**
 *
 */
UUIFrameworkUserWidget::UUIFrameworkUserWidget()
	: ReplicatedNamedSlotList(this)
{
}

void UUIFrameworkUserWidget::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ReplicatedNamedSlotList, Params);
}

bool UUIFrameworkUserWidget::LocalIsReplicationReady() const
{
	return Super::LocalIsReplicationReady() && !WidgetClass.IsNull();
}

void UUIFrameworkUserWidget::SetWidgetClass(TSoftClassPtr<UWidget> InWidgetClass)
{
	WidgetClass = InWidgetClass;
	MARK_PROPERTY_DIRTY_FROM_NAME(UUIFrameworkWidget, WidgetClass, this);
}

void UUIFrameworkUserWidget::SetNamedSlot(FName SlotName, UUIFrameworkWidget* Widget)
{
	if (Widget == nullptr || SlotName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("The widget is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToAdd");
	}
	else
	{
		// Reset the widget to make sure the id is set and it may have been duplicated during the attach
		FUIFrameworkUserWidgetNamedSlot Entry;
		Entry.SlotName = SlotName;
		Entry.AuthoritySetWidget(Widget);
		Entry.AuthoritySetWidget(FUIFrameworkModule::AuthorityAttachWidget(this, Entry.AuthorityGetWidget()));
		ReplicatedNamedSlotList.AuthorityAddEntry(Entry);
	}
}

UUIFrameworkWidget* UUIFrameworkUserWidget::GetNamedSlot(FName SlotName) const
{
	UUIFrameworkWidget* Result = nullptr;
	if (SlotName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("The slot name is invalid. It can't be added."), ELogVerbosity::Warning, "InvalidWidgetToGet");
	}
	else
	{
		if (const FUIFrameworkUserWidgetNamedSlot* Slot = ReplicatedNamedSlotList.AuthorityFindEntry(SlotName))
		{
			Result = Slot->AuthorityGetWidget();
		}

	}
	return Result;
}

void UUIFrameworkUserWidget::AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
{
	Super::AuthorityForEachChildren(Func);

	ReplicatedNamedSlotList.AuthorityForEachChildren(Func);
}

void UUIFrameworkUserWidget::AuthorityRemoveChild(UUIFrameworkWidget* Widget)
{
	Super::AuthorityRemoveChild(Widget);

	ReplicatedNamedSlotList.AuthorityRemoveEntry(Widget);
}

void UUIFrameworkUserWidget::LocalAddChild(FUIFrameworkWidgetId ChildId)
{
	bool bIsAdded = false;
	if (FUIFrameworkUserWidgetNamedSlot* NamedSlotEntry = ReplicatedNamedSlotList.FindEntry(ChildId))
	{
		if (FUIFrameworkWidgetTree* WidgetTree = GetWidgetTree())
		{
			if (UUIFrameworkWidget* ChildWidget = WidgetTree->FindWidgetById(ChildId))
			{
				UWidget* ChildUMGWidget = ChildWidget->LocalGetUMGWidget();
				if (ensure(ChildUMGWidget))
				{
					NamedSlotEntry->LocalAquireWidget();
					if (UUserWidget* LocalUMGUserWidget = Cast<UUserWidget>(LocalGetUMGWidget()))
					{
						LocalUMGUserWidget->SetContentForSlot(NamedSlotEntry->SlotName, ChildUMGWidget);
					}
					else
					{
						UE_LOGF(LogUIFramework, Log, "Can't set the NamedSlot on widget '%ls' because it is not a UserWidget.", *ChildUMGWidget->GetName());
						Super::LocalAddChild(ChildId);
					}
				}
				else
				{
					UE_LOG(LogUIFramework, Error, TEXT("The widget '%" INT64_FMT "' is invalid."), ChildId.GetKey());
					Super::LocalAddChild(ChildId);
				}
			}
			else
			{
				UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
				Super::LocalAddChild(ChildId);
			}
		}
		else
		{
			UE_LOG(LogUIFramework, Log, TEXT("The widget '%" INT64_FMT "' doesn't exist in the WidgetTree."), ChildId.GetKey());
			Super::LocalAddChild(ChildId);
		}
	}
	else
	{
		UE_LOG(LogUIFramework, Verbose, TEXT("The widget '%" INT64_FMT "' was not found in the Canvas Slots."), ChildId.GetKey());
		Super::LocalAddChild(ChildId);
	}
}
