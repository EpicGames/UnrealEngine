// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIFrameworkButtonWidget.h"

#include "SUIFrameworkButtonWidget.h"

#include "CommonActivatableWidget.h"
#include "CommonUITypes.h"
#include "Components/Button.h"
#include "Components/ButtonSlot.h"
#include "EnhancedActionKeyMapping.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#include "Input/CommonUIActionRouterBase.h"
#include "Input/CommonUIInputTypes.h"
#include "InputTriggers.h"
#include "TimerManager.h"
#include "UIFWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIFrameworkButtonWidget)

UUIFrameworkButtonWidget::UUIFrameworkButtonWidget()
{
	FButtonStyle TempStyle = GetStyle();
	TempStyle.NormalPadding = FMargin(0.0f);
	TempStyle.PressedPadding = FMargin(0.0f);
	TempStyle.Normal.DrawAs = ESlateBrushDrawType::NoDrawType;
	TempStyle.Hovered.DrawAs = ESlateBrushDrawType::NoDrawType;
	TempStyle.Pressed.DrawAs = ESlateBrushDrawType::NoDrawType;
	SetStyle(TempStyle);
}

TSharedRef<SWidget> UUIFrameworkButtonWidget::RebuildWidget()
{	
	MyButton = UIFButton = SNew(SUIFrameworkButtonWidget)
		.OnTouchLongPressedDelegate(BIND_UOBJECT_DELEGATE(FOnTouch, SlateHandleTouchLongPress))
		.OnTouchStartedDelegate(BIND_UOBJECT_DELEGATE(FOnTouch, SlateHandleTouchStarted))
		.OnTouchEndedDelegate(BIND_UOBJECT_DELEGATE(FOnTouch, SlateHandleTouchEnded))
		.OnClicked(BIND_UOBJECT_DELEGATE(FOnClicked, SlateHandleClicked))
		.OnPressed(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandlePressed))
		.OnReleased(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleReleased))
		.OnHovered_UObject(this, &ThisClass::SlateHandleHovered)
		.OnUnhovered_UObject(this, &ThisClass::SlateHandleUnhovered)
		.OnReceivedFocus_UObject(this, &ThisClass::SlateHandleOnReceivedFocus)
		.OnLostFocus_UObject(this, &ThisClass::SlateHandleOnLostFocus)
		.OnSlateButtonDragDetected(BIND_UOBJECT_DELEGATE(FOnDragDetected, SlateHandleDragDetected))
		.OnSlateButtonDragEnter(BIND_UOBJECT_DELEGATE(FOnDragEnter, SlateHandleDragEnter))
		.OnSlateButtonDragLeave(BIND_UOBJECT_DELEGATE(FOnDragLeave, SlateHandleDragLeave))
		.OnSlateButtonDragOver(BIND_UOBJECT_DELEGATE(FOnDragOver, SlateHandleDragOver))
		.OnSlateButtonDrop(BIND_UOBJECT_DELEGATE(FOnDrop, SlateHandleDrop))
		.ButtonStyle(&GetStyle())
		.ClickMethod(GetClickMethod())
		.TouchMethod(GetTouchMethod())
		.PressMethod(GetPressMethod())
		.IsFocusable(GetIsFocusable())
		.AllowDragDrop(bAllowDragDrop);

	OnClicked.AddUniqueDynamic(this, &UUIFrameworkButtonWidget::HandleButtonClicked);

	if (GetChildrenCount() > 0)
	{
		Cast<UButtonSlot>(GetContentSlot())->BuildSlot(MyButton.ToSharedRef());
	}

	if (!InputPreProcessor)
	{
		InputPreProcessor = MakeShared<FUIFInputPreProcessor>();
		InputPreProcessor->OnTrackedTouchEnded().BindUObject(this, &UUIFrameworkButtonWidget::HandleTouchEnded);
	}

	return UIFButton.ToSharedRef();
}

void UUIFrameworkButtonWidget::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	if (UWorld* World = GetWorld())
	{
		FTimerManager& TimerManager = World->GetTimerManager();
		TimerManager.SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UUIFrameworkButtonWidget::BindTriggeringInputActionToClick));
	}
}

void UUIFrameworkButtonWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	UnbindTriggeringInputActionToClick();
	UIFButton.Reset();

	if (InputPreProcessor)
	{
		InputPreProcessor->OnTrackedTouchEnded().Unbind();
		InputPreProcessor->StopTrackingTouch();
	}

	InputPreProcessor.Reset();
}

void UUIFrameworkButtonWidget::HandleButtonClicked()
{
	OnTriggered.Broadcast();
	ExecuteTriggeredInput();
}

void UUIFrameworkButtonWidget::ExecuteTriggeredInput()
{
	// Fire all the bound input action events
	if (TriggeringInputAction)
	{
		if (APlayerController* PlayerController = GetOwningPlayer())
		{
			if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
			{
				if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
				{
					if (const UEnhancedPlayerInput* const PlayerInput = EnhancedInputLocalPlayerSubsystem->GetPlayerInput())
					{
						// Confirm that this input action is active before injecting it
						if (PlayerInput->FindActionInstanceData(TriggeringInputAction.Get()))
						{
							FInputActionValue RawValue = FInputActionValue(true);
							CommonUI::InjectEnhancedInputForAction(LocalPlayer, TriggeringInputAction.Get(), RawValue);
						}
					}
				}
			}
		}
	}
}

void UUIFrameworkButtonWidget::BindTriggeringInputActionToClick()
{
	if (!TriggeringInputAction)
	{
		return;
	}

	if (!TriggeringBindingHandle.IsValid())
	{
		if (APlayerController* PlayerController = GetOwningPlayer())
		{
			TSharedPtr<SWidget> CurWidget = GetCachedWidget();
			ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer();

			if (!LocalPlayer || !CurWidget)
			{
				return;
			}

			// Register a UI Action Binding for the input action so that it can be triggered in Menu mode
			if (const UCommonActivatableWidget* ActivatableWidget = UCommonUIActionRouterBase::FindOwningActivatable(CurWidget, LocalPlayer))
			{
				if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*ActivatableWidget))
				{
					FBindUIActionArgs BindArgs(TriggeringInputAction.Get(), false, FSimpleDelegate::CreateUObject(this, &UUIFrameworkButtonWidget::HandleButtonClicked));
					BindArgs.bConsumeInput = TriggeringInputAction->bConsumeInput;
					BindArgs.PriorityWithinCollection = 1;

					// This is a bit of a hack to enforce Pressed trigger if it is the only trigger in an input action.
					// We need this to be able to consume and block other actions
					const bool bHasPressedTrigger = TriggeringInputAction->Triggers.ContainsByPredicate(
						[](const UInputTrigger* Trigger)
						{
							return Cast<UInputTriggerPressed>(Trigger) != nullptr;
						}
					);
					bool ShouldConsumePressed = TriggeringInputAction->bConsumesActionAndAxisMappings && bHasPressedTrigger && TriggeringInputAction->Triggers.Num() == 1;

					BindArgs.KeyEvent = GetClickMethod() == EButtonClickMethod::MouseDown || ShouldConsumePressed ? IE_Pressed : IE_Released;
					TriggeringBindingHandle = ActionRouter->RegisterUIActionBinding(*this, BindArgs);
				}
			}
		}
	}
}

APlayerController* UUIFrameworkButtonWidget::GetOwningPlayer() const
{
	if (APlayerController* PlayerController = Super::GetOwningPlayer())
	{
		return PlayerController;
	}
	else if (UUIFrameworkWidget* FrameworkWidget = Cast<UUIFrameworkWidget>(GetOuter()))
	{
		return FrameworkWidget->GetPlayerController();
	}

	return nullptr;
}

void UUIFrameworkButtonWidget::UnbindTriggeringInputActionToClick()
{
	if (TriggeringBindingHandle.IsValid())
	{
		TriggeringBindingHandle.Unregister();
	}
}

void UUIFrameworkButtonWidget::SetTriggeringEnhancedInputAction(UInputAction* InInputAction)
{
	if (TriggeringInputAction != InInputAction)
	{
		UnbindTriggeringInputActionToClick();

		TriggeringInputAction = InInputAction;

		if (!IsDesignTime())
		{
			BindTriggeringInputActionToClick();
		}
	}
}

FReply UUIFrameworkButtonWidget::SlateHandleTouchLongPress(const FPointerEvent& PointerEvent)
{
	if (UIFButton)
	{
		bLongPressActive = true;
		OnTouchLongPress.ExecuteIfBound();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply UUIFrameworkButtonWidget::SlateHandleTouchStarted(const FPointerEvent& PointerEvent)
{
	bTouchStarted = true;
	StartTrackingTouch(PointerEvent.GetPointerIndex());
	OnTouchStarted.ExecuteIfBound();

	return FReply::Unhandled();	// let the button content process the mouse down event too.
}

FReply UUIFrameworkButtonWidget::SlateHandleTouchEnded(const FPointerEvent& PointerEvent)
{
	return HandleTouchEnded() ? FReply::Handled() : FReply::Unhandled();
}

bool UUIFrameworkButtonWidget::HandleTouchEnded()
{
	const bool bShouldPreventClick = bLongPressActive;
	bTouchStarted = false;
	StopTrackingTouch();
	if (bLongPressActive)
	{
		OnTouchEnded.ExecuteIfBound();
		bLongPressActive = false;
	}
	return bShouldPreventClick;
}

void UUIFrameworkButtonWidget::StartTrackingTouch(int32 PointerIndex)
{
	if (InputPreProcessor)
	{
		InputPreProcessor->StartTrackingTouch(PointerIndex);
	}
}

void UUIFrameworkButtonWidget::StopTrackingTouch()
{
	if (InputPreProcessor)
	{
		InputPreProcessor->StopTrackingTouch();
	}
}

bool FUIFInputPreProcessor::HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetPointerIndex() == TrackedTouchIndex && MouseEvent.IsTouchEvent() && OnTrackedTouchEndedDelegate.IsBound())
	{
		return OnTrackedTouchEndedDelegate.Execute();
	}
	return false;
}

void FUIFInputPreProcessor::StartTrackingTouch(int32 Index)
{
	TrackedTouchIndex = Index;
	FSlateApplication::Get().RegisterInputPreProcessor(AsShared());
}

void FUIFInputPreProcessor::StopTrackingTouch()
{
	FSlateApplication::Get().UnregisterInputPreProcessor(AsShared());
	TrackedTouchIndex = INDEX_NONE;
}