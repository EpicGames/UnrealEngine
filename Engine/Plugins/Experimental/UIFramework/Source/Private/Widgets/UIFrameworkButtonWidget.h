// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Button.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/IInputProcessor.h"
#include "Input/UIActionBindingHandle.h"
#include "UIFrameworkButtonWidget.generated.h"

#define UE_API UIFRAMEWORK_API

class FUIFInputPreProcessor;

DECLARE_DELEGATE_RetVal(
	bool,
	FTrackedTouchEnded)

class UInputAction;
class UPlayerController;

UCLASS(MinimalAPI)
class UUIFrameworkButtonWidget : public UButton
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkButtonWidget();

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual APlayerController* GetOwningPlayer() const override;
	// End of UWidget interface

	void SetTriggeringEnhancedInputAction(UInputAction* InInputAction);

	FSimpleDelegate OnTouchLongPress;
	FSimpleDelegate OnTouchStarted;
	FSimpleDelegate OnTouchEnded;

	FOnButtonClickedEvent OnTriggered;

protected:
	UE_API FReply SlateHandleTouchLongPress(const FPointerEvent& PointerEvent);
	UE_API FReply SlateHandleTouchStarted(const FPointerEvent& PointerEvent);
	UE_API FReply SlateHandleTouchEnded(const FPointerEvent& PointerEvent);

private:

	UFUNCTION()
	void HandleButtonClicked();
	void ExecuteTriggeredInput();

	void BindTriggeringInputActionToClick();
	void UnbindTriggeringInputActionToClick();

	void StartTrackingTouch(int32 PointerIndex);
	void StopTrackingTouch();

	bool HandleTouchEnded();

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input)
	TObjectPtr<UInputAction> TriggeringInputAction;

private:
	/** Cached pointer to the underlying slate button owned by this UWidget */
	TSharedPtr<class SUIFrameworkButtonWidget> UIFButton;

	FUIActionBindingHandle TriggeringBindingHandle;

	TSharedPtr<FUIFInputPreProcessor> InputPreProcessor;
	bool bTouchStarted = false;
	bool bLongPressActive = false;
};

class FUIFInputPreProcessor
	: public IInputProcessor
	, public TSharedFromThis<FUIFInputPreProcessor>
{
public:
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}
	virtual bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;

	FTrackedTouchEnded& OnTrackedTouchEnded() { return OnTrackedTouchEndedDelegate; }

	void StartTrackingTouch(int32 Index);
	void StopTrackingTouch();

	int32 GetTrackedTouchIndex() const { return TrackedTouchIndex; }

private:
	int32 TrackedTouchIndex = INDEX_NONE;
	FTrackedTouchEnded OnTrackedTouchEndedDelegate;
};

#undef UE_API
