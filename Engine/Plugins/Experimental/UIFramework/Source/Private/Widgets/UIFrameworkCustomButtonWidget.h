// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UIFrameworkButtonWidget.h"
#include "UIFrameworkCustomButtonWidget.generated.h"

struct FStreamableHandle;
class SBorder;
class STextBlock;
class SButton;

/**
 * 
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Custom Button"))
class UUIFrameworkCustomButtonWidget : public UUIFrameworkButtonWidget
{
	GENERATED_BODY()
public:
	UUIFrameworkCustomButtonWidget();
	
	virtual TSharedRef<SWidget> RebuildWidget() override;
	
private:
	void LocalCreateDefaultUMGWidget();
	
	virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	
	UFUNCTION()
	void HandleButtonClick();
	
	UFUNCTION()
	void HandleButtonHighlight();
	
	UFUNCTION()
	void HandleButtonUnhighlight();
	
	UFUNCTION()
	void HandleButtonHover();
	
	UFUNCTION()
	void HandleButtonUnhover();
	
	UPROPERTY(Transient)
	TObjectPtr<UWidget> LocalDefaultUMGWidget;
	
	UPROPERTY(EditDefaultsOnly, Category = "UI Framework")
	TSoftClassPtr<UWidget> DefaultContentWidgetClass;
	
	TSharedPtr<FStreamableHandle> WidgetClassStreamableHandle;
	
protected:
	TSharedPtr<FStreamableHandle> AsyncLoadDefaultWidgetClass();

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Clicked"))
	FOnButtonClickedEvent OnButtonClicked;
	
	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Highlight"))
	FOnButtonClickedEvent OnButtonHighlight;
	
	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Unhighlight"))
	FOnButtonClickedEvent OnButtonUnhighlight;
};

