// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UIFrameworkCustomButtonWidget.h"
#include "Components/NamedSlot.h"
#include "UIFrameworkTouchActionWrapperWidget.generated.h"

class FUIFTouchInputPreProcessor;
class UInputAction;

/**
 * 
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Touch Action Wrapper"))
class UUIFrameworkTouchActionWrapperWidget : public UUIFrameworkButtonWidget
{
	GENERATED_BODY()
	
public:
	UUIFrameworkTouchActionWrapperWidget();
	
	bool HandleTouch() const;
private:
	
	virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	virtual TSharedRef<SWidget> RebuildWidget() override;
	
	TSharedPtr<FUIFTouchInputPreProcessor> TouchInputPreProcessor;
};

class FUIFTouchInputPreProcessor
	: public IInputProcessor
	, public TSharedFromThis<FUIFTouchInputPreProcessor>
{
public:
	FUIFTouchInputPreProcessor();
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}
	virtual bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;

	FTrackedTouchEnded& OnTrackedTouchEnded() { return OnTrackedTouchEndedDelegate; }
	void SetTargetWidget(TWeakPtr<SWidget> InWidget);
	
	void StartTrackingTouch();
	void StopTrackingTouch();

private:
	
	bool IsInputHittingWidget(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent);
	
	
	FTrackedTouchEnded OnTrackedTouchEndedDelegate;
	
	TWeakPtr<SWidget> TargetWidget;
	
	TArray<FSoftObjectPath> WidgetClassesToIgnore;
	TArray<TWeakPtr<SWidget>> CachedWidgetsToIgnore;
};