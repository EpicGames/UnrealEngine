// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UIFEvents.generated.h"


class APlayerController;
class UUIFrameworkWidget;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkSimpleEventArgument
{
	GENERATED_BODY()

	FUIFrameworkSimpleEventArgument() = default;

	UPROPERTY()
	TObjectPtr<APlayerController> PlayerController;

	UPROPERTY()
	TObjectPtr<UUIFrameworkWidget> Sender;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FUIFrameworkSimpleEvent, FUIFrameworkSimpleEventArgument);

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkClickEventArgument : public FUIFrameworkSimpleEventArgument
{
	GENERATED_BODY()

	FUIFrameworkClickEventArgument() = default;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FUIFrameworkClickEvent, FUIFrameworkClickEventArgument);

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkHighlightEventArgument : public FUIFrameworkSimpleEventArgument
{
	GENERATED_BODY()

	FUIFrameworkHighlightEventArgument() = default;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FUIFrameworkHighlightEvent, FUIFrameworkHighlightEventArgument);

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkUnhighlightEventArgument : public FUIFrameworkSimpleEventArgument
{
	GENERATED_BODY()

	FUIFrameworkUnhighlightEventArgument() = default;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FUIFrameworkUnhighlightEvent, FUIFrameworkUnhighlightEventArgument);