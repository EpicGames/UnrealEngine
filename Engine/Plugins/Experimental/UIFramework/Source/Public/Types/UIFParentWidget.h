// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIFParentWidget.generated.h"

#define UE_API UIFRAMEWORK_API

class UObject;
class UUIFrameworkPlayerComponent;
class UUIFrameworkWidget;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkParentWidget
{
	GENERATED_BODY()

	FUIFrameworkParentWidget() = default;

	UE_API FUIFrameworkParentWidget(UUIFrameworkWidget* InWidget);
	UE_API FUIFrameworkParentWidget(UUIFrameworkPlayerComponent* InPlayer);

public:
	bool IsParentValid() const
	{
		return Parent != nullptr;
	}

	bool IsWidget() const
	{
		check(IsParentValid());
		return bIsParentAWidget;
	}

	bool IsPlayerComponent() const
	{
		check(IsParentValid());
		return !bIsParentAWidget;
	}

	UE_API UUIFrameworkWidget* AsWidget() const;
	UE_API UUIFrameworkPlayerComponent* AsPlayerComponent() const;

	UE_API bool operator== (const UUIFrameworkWidget* Other) const;

	bool operator!= (const UUIFrameworkWidget* Other) const
	{
		return !((*this) == Other);
	}

	bool operator== (const FUIFrameworkParentWidget& Other) const
	{
		return Other.Parent == Parent;
	}

	bool operator!= (const FUIFrameworkParentWidget& Other) const
	{
		return Other.Parent != Parent;
	}

private:
	UPROPERTY()
	TObjectPtr<UObject> Parent = nullptr;

	bool bIsParentAWidget = false;
};

#undef UE_API
