// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "Types/UIFWidgetId.h"

#include "UIFSlotBase.generated.h"

#define UE_API UIFRAMEWORK_API

enum EHorizontalAlignment : int;
enum EVerticalAlignment : int;

class UUIFrameworkWidget;
class UWidget;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkSlotBase : public FFastArraySerializerItem
{
	GENERATED_BODY()

	FUIFrameworkSlotBase() = default;

	UUIFrameworkWidget* AuthorityGetWidget() const
	{
		return Widget;
	}

	UE_API void AuthoritySetWidget(UUIFrameworkWidget* Widget);

	FUIFrameworkWidgetId GetWidgetId() const
	{
		return WidgetId;
	}

	void LocalAquireWidget()
	{
		LocalPreviousWidgetId = WidgetId;
	}

	bool LocalIsAquiredWidgetValid() const
	{
		return LocalPreviousWidgetId == WidgetId;
	}

private:
	UPROPERTY(BlueprintReadWrite, Category = "UI Framework", NotReplicated, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UUIFrameworkWidget> Widget = nullptr;

	UPROPERTY()
	FUIFrameworkWidgetId WidgetId;

	/**
	 * The widget that was previously added on the local UMG Widget.
	 * The server may have changed it but the "application" of that modification may be applied on the next frame by the PlayerComponent.
	 */
	UPROPERTY(NotReplicated)
	FUIFrameworkWidgetId LocalPreviousWidgetId;
};


/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkSimpleSlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	FUIFrameworkSimpleSlot() = default;

	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	FMargin Padding;

	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;

	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = EVerticalAlignment::VAlign_Fill;
};

#undef UE_API
