// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIFWidgetId.generated.h"

#define UE_API UIFRAMEWORK_API

class UUIFrameworkWidget;

/**
 *
 */
USTRUCT()
struct FUIFrameworkWidgetId
{
	GENERATED_BODY()

	FUIFrameworkWidgetId() = default;
	UE_API explicit FUIFrameworkWidgetId(UUIFrameworkWidget* InWidget);

	static UE_API FUIFrameworkWidgetId MakeNew();
	static UE_API FUIFrameworkWidgetId MakeRoot();

private:
	FUIFrameworkWidgetId(int64 InKey)
		: Key(InKey)
	{}

public:
	int64 GetKey() const
	{
		return Key;
	}

	bool IsRoot() const
	{
		return Key == 0;
	}
	
	bool IsValid() const
	{
		return Key != INDEX_NONE;
	}

	bool operator==(const FUIFrameworkWidgetId& Other) const
	{
		return Key == Other.Key;
	}

	bool operator!=(const FUIFrameworkWidgetId& Other) const
	{
		return Key != Other.Key;
	}
	
	inline friend uint32 GetTypeHash(const FUIFrameworkWidgetId& Value)
	{
		return GetTypeHash(Value.Key);
	}

private:
	UPROPERTY()
	int64 Key = INDEX_NONE;

	static UE_API int64 KeyGenerator;
};

#undef UE_API
