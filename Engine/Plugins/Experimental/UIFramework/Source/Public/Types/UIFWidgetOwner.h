// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UIFWidgetOwner.generated.h"

class UGameInstance;
class APlayerController;
class UWorld;

/**
 *
 */
USTRUCT()
struct FUIFrameworkWidgetOwner
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<APlayerController> PlayerController = nullptr;

	UPROPERTY()
	TObjectPtr<UGameInstance> GameInstance = nullptr;

	UPROPERTY()
	TObjectPtr<UWorld> World = nullptr;
};
