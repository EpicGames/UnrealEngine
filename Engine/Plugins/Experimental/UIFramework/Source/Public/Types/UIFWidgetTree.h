// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Serialization/FastArraySerializer.h"
#include "Types/UIFWidgetId.h"
#include "UObject/ObjectKey.h"

#include "UIFWidgetTree.generated.h"

#define UE_API UIFRAMEWORK_API

class IUIFrameworkWidgetTreeOwner;

#ifndef UE_UIFRAMEWORK_WITH_DEBUG
	#define UE_UIFRAMEWORK_WITH_DEBUG !(UE_BUILD_SHIPPING)
#endif

struct FReplicationFlags;
struct FUIFrameworkWidgetTree;
struct FUIFrameworkWidgetTreeEntry;
class AActor;
class FOutBunch;
class UActorChannel;
class UUIFrameworkWidget;

#if UE_UIFRAMEWORK_WITH_DEBUG
class UWidget;
#endif

/**
 *
 */
USTRUCT()
struct FUIFrameworkWidgetTreeEntry : public FFastArraySerializerItem
{
	GENERATED_BODY()

	FUIFrameworkWidgetTreeEntry() = default;
	UE_API FUIFrameworkWidgetTreeEntry(UUIFrameworkWidget* Parent, UUIFrameworkWidget* Child);

	UE_API bool IsParentValid() const;
	UE_API bool IsChildValid() const;

	UPROPERTY()
	TObjectPtr<UUIFrameworkWidget> Parent = nullptr;

	UPROPERTY()
	TObjectPtr<UUIFrameworkWidget> Child = nullptr;
	
	UPROPERTY()
	FUIFrameworkWidgetId ParentId;

	UPROPERTY()
	FUIFrameworkWidgetId ChildId;

	UE_API FString GetDebugString();
};


/**
 * A valid snapshot of the widget tree that can be replicated to local instance.
 * Authority widgets know their parent/children relation. That information is not replicated to the local widgets.
 * When a widget is added to the tree, the tree is updated. The widget now has to inform the tree when that relationship changes until it's remove from the tree.
 */
USTRUCT()
struct FUIFrameworkWidgetTree : public FFastArraySerializer
{
	GENERATED_BODY()

public:
	FUIFrameworkWidgetTree() = default;
	UE_API FUIFrameworkWidgetTree(AActor* InReplicatedOwner, IUIFrameworkWidgetTreeOwner* InOwner);
	UE_API ~FUIFrameworkWidgetTree();

public:
	//~ Begin of FFastArraySerializer
	UE_API void PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize);
	UE_API void PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize);
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkWidgetTreeEntry, FUIFrameworkWidgetTree>(Entries, DeltaParms, *this);
	}

	AActor* GetReplicationOwner() const
	{
		return ReplicatedOwner;
	}

	UE_API bool ReplicateSubWidgets(UActorChannel* Channel, FOutBunch* Bunch, FReplicationFlags* RepFlags);

	/** Add a new widget to the top hierarchy. */
	UE_API void AuthorityAddRoot(UUIFrameworkWidget* Widget);
	/**
	 * Change the parent / child relationship of the child widget.
	 * If the child widget had a parent, that relationship entry will replaced it by a new one.
	 */
	UE_API void AuthorityAddWidget(UUIFrameworkWidget* Parent, UUIFrameworkWidget* Child);
	/**
	 * Remove the widget and all of its children and grand-children from the tree.
	 * It will clean all the parent relationship from the tree.
	 */
	UE_API void AuthorityRemoveWidgetAndChildren(UUIFrameworkWidget* Widget);

	/**
	 * The widget was removed from the client and the Authority is not aware of it.
	 */
	UE_API void LocalRemoveRoot(const UUIFrameworkWidget* Widget);

	//~ It is not safe to use the ReplicationId on the Authority because any add or remove would clear the ItemMap.
	UE_API FUIFrameworkWidgetTreeEntry* LocalGetEntryByReplicationId(int32 WidgetId);
	UE_API const FUIFrameworkWidgetTreeEntry* LocalGetEntryByReplicationId(int32 WidgetId) const;

	/** Find the widget by its unique Id. The widget needs to be in the Tree. */
	UE_API UUIFrameworkWidget* FindWidgetById(FUIFrameworkWidgetId WidgetId);
	UE_API const UUIFrameworkWidget* FindWidgetById(FUIFrameworkWidgetId WidgetId) const;

	/** Add all widgets in the tree to the ActorChannel replicated list */
	UE_API void AuthorityAddAllWidgetsFromActorChannel();

	/** Removes all widgets added to the ActorChannel replicated list */
	UE_API void AuthorityRemoveAllWidgetsFromActorChannel();

	/** Gets the root widget of the tree given the Id of a widget in the tree */
	UE_API const FUIFrameworkWidgetTreeEntry* FindRootEntryById(FUIFrameworkWidgetId WidgetId) const;

#if UE_UIFRAMEWORK_WITH_DEBUG
	UE_API void AuthorityTest() const;
	UE_API void LogTree() const;
	UE_API void LogWidgetsChildren() const;
#endif

	DECLARE_MULTICAST_DELEGATE_OneParam(FUIFrameworkWidgetDelegate, UUIFrameworkWidget*);
	FUIFrameworkWidgetDelegate AuthorityOnWidgetAdded;
	FUIFrameworkWidgetDelegate AuthorityOnWidgetRemoved;
	FUIFrameworkWidgetDelegate LocalOnWidgetAdded;

private:
	UE_API void AuthorityAddChildInternal(UUIFrameworkWidget* Parent, UUIFrameworkWidget* Child, bool bFirst);
	UE_API void AuthorityAddChildRecursiveInternal(UUIFrameworkWidget* Widget);
	UE_API bool AuthorityRemoveChildRecursiveInternal(UUIFrameworkWidget* Widget);

#if UE_UIFRAMEWORK_WITH_DEBUG
	UE_API void LogTreeInternal(const FUIFrameworkWidgetTreeEntry& Entry, FString Spaces) const;
	UE_API void AuthorityLogWidgetsChildrenInternal(UUIFrameworkWidget* Widget, FString Spaces) const;
	UE_API void LocalLogWidgetsChildrenInternal(const UWidget* Widget, FString Spaces) const;
#endif

private:
	UPROPERTY()
	TArray<FUIFrameworkWidgetTreeEntry> Entries;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<AActor> ReplicatedOwner;

	TMap<FObjectKey, int32> AuthorityIndexByWidgetMap;
	TMap<FUIFrameworkWidgetId, TWeakObjectPtr<UUIFrameworkWidget>> WidgetByIdMap;
	IUIFrameworkWidgetTreeOwner* Owner = nullptr;
};

template<>
struct TStructOpsTypeTraits<FUIFrameworkWidgetTree> : public TStructOpsTypeTraitsBase2<FUIFrameworkWidgetTree>
{
	enum { WithNetDeltaSerializer = true };
};

#undef UE_API
