// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h" // IWYU pragma: keep

struct FUIFrameworkWidgetOwner;

class APlayerController;
class UUIFrameworkWidget;
struct FUIFrameworkWidgetTree;
struct FUIFrameworkWidgetTreeEntry;

class IUIFrameworkWidgetTreeOwner
{
public:
	/** The widget tree. */
	virtual FUIFrameworkWidgetTree& GetWidgetTree() = 0;

	/** The object that will be used to create the UserWidget. */
	virtual FUIFrameworkWidgetOwner GetWidgetOwner() const = 0;

	/** A widget was added to the tree. */
	virtual void LocalWidgetWasAddedToTree(const FUIFrameworkWidgetTreeEntry& Entry)
	{
	}

	/** A widget was removed to the tree. */
	virtual void LocalWidgetRemovedFromTree(const FUIFrameworkWidgetTreeEntry& Entry)
	{
	}

	/** Remove the widget (and the child) from the server. */
	virtual void LocalRemoveWidgetRootFromTree(const UUIFrameworkWidget* Widget)
	{
	}
};
