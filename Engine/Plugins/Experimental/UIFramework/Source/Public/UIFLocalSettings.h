// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UIFLocalSettings.generated.h"

#define UE_API UIFRAMEWORK_API

class UObject;

/**
 * 
 */
UCLASS(MinimalAPI, config=Game, defaultconfig, meta = (DisplayName = "UI Framework Local Settings"))
class UUIFrameworkLocalSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkLocalSettings();

	UObject* GetErrorResource() const
	{
		return ErrorResourcePtr;
	}

	UObject* GetLoadingResource() const
	{
		return LoadingResourcePtr;
	}

	UE_API void LoadResources() const;
	
	UE_API virtual FName GetCategoryName() const override;

	virtual bool NeedsLoadForServer() const override
	{
		return false;
	}

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

private:
	/**
	 * The image to render for when a requested resource is inaccessible.
	 * It can be a UTexture or UMaterialInterface or an object implementing the AtlasedTextureInterface.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI Framework", meta = (AllowPrivateAccess = "true", DisplayThumbnail = "true", AllowedClasses = "/Script/Engine.Texture,/Script/Engine.MaterialInterface,/Script/Engine.SlateTextureAtlasInterface", DisallowedClasses = "/Script/MediaAssets.MediaTexture"))
	TSoftObjectPtr<UObject> ErrorResource;

	/**
	 * The image to render while a resource is loading.
	 * It can be a UTexture or UMaterialInterface or an object implementing the AtlasedTextureInterface.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI Framework", meta = (AllowPrivateAccess = "true", DisplayThumbnail = "true", AllowedClasses = "/Script/Engine.Texture,/Script/Engine.MaterialInterface,/Script/Engine.SlateTextureAtlasInterface", DisallowedClasses = "/Script/MediaAssets.MediaTexture"))
	TSoftObjectPtr<UObject> LoadingResource;

	//~ kept alive with by AddToRoot
	TObjectPtr<UObject> ErrorResourcePtr;
	TObjectPtr<UObject> LoadingResourcePtr;
	mutable bool bResourceLoaded = false;
};

#undef UE_API
