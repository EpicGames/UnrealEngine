// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkParentWidget;
template <class TClass> class TSubclassOf;

class UUIFrameworkPlayerComponent;
class UUIFrameworkPresenter;
class UUIFrameworkWidget;

/**
 * 
 */
class FUIFrameworkModule : public IModuleInterface
{
public:
	/**
	 * Set the new widget parent.
	 * It will attach the widget to the correct ReplicationOwner and add it to the WidgetTree.
	 * If the ReplicationOwner cannot be changed, a duplicate of the widget will be created and the AuthorityOnReattachWidgets will be broadcast.
	 * This function is recursive if the owner changed.
	 */
	static UE_API UUIFrameworkWidget* AuthorityAttachWidget(FUIFrameworkParentWidget Parent, UUIFrameworkWidget* Child);
	static UE_API bool AuthorityCanWidgetBeAttached(FUIFrameworkParentWidget Parent, UUIFrameworkWidget* Child);
	/**
	 * Will remove the widget from the tree and the replication owner.
	 */
	static UE_API void AuthorityDetachWidgetFromParent(UUIFrameworkWidget* Child);

	//~ this should be a project setting
	static UE_API void SetPresenterClass(TSubclassOf<UUIFrameworkPresenter> Director);
	static UE_API TSubclassOf<UUIFrameworkPresenter> GetPresenterClass();

private:
	static void AuthorityDetachWidgetFromParentInternal(UUIFrameworkWidget* Child, bool bTemporary);
	//static UUIFrameworkWidget* AuthorityRenameRecursive(UUIFrameworkPlayerComponent* ReplicationOwner, UUIFrameworkWidget* Widget, UObject* NewOuter);
	//static void AuthoritySetParentReplicationOwnerRecursive(UUIFrameworkWidget* Widget);
};

#undef UE_API
