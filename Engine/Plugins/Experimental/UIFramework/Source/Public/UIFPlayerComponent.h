// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "GameFramework/PlayerController.h"
#include "Types/UIFSlotBase.h"
#include "Types/UIFWidgetTree.h"

#include "Types/UIFWidgetTreeOwner.h"
#include "UIFPlayerComponent.generated.h"

#define UE_API UIFRAMEWORK_API

class UUIFrameworkPlayerComponent;
class UUIFrameworkPresenter;
class UUIFrameworkWidget;
class UWidget;
struct FStreamableHandle;

DECLARE_MULTICAST_DELEGATE(FOnPendingReplicationProcessed);

/**
 *
 */
UENUM(BlueprintType)
enum class EUIFrameworkGameLayerType : uint8
{
	Viewport,
	PlayerScreen,
};

/**
 *
 */
UENUM(BlueprintType)
enum class EUIFrameworkInputMode : uint8
{
	// Input is received by the UI.
	UI,
	// Input is received by the Game.
	Game,
};


/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkGameLayerSlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	FUIFrameworkGameLayerSlot() = default;

	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	int32 ZOrder = 0;
	
	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	EUIFrameworkInputMode InputMode = EUIFrameworkInputMode::Game;

	UPROPERTY(BlueprintReadWrite, Category = "UI Framework")
	EUIFrameworkGameLayerType Type = EUIFrameworkGameLayerType::Viewport;
};

/**
 *
 */
USTRUCT()
struct FUIFrameworkWidgetToFocus
{
	GENERATED_BODY()

	//~ ID of the widget to focus on.
	UPROPERTY()
	FUIFrameworkWidgetId Id;

	//~ Tracker to ensure widget ID is replicated even if it hasn't changed.
	UPROPERTY()
	int32 ReplicationCounter = 0;
};

/**
 *
 */
USTRUCT()
struct FUIFrameworkGameLayerSlotList : public FFastArraySerializer
{
	GENERATED_BODY()

	friend UUIFrameworkPlayerComponent;

	FUIFrameworkGameLayerSlotList() = default;
	FUIFrameworkGameLayerSlotList(UUIFrameworkPlayerComponent* InOwner)
		: Owner(InOwner)
	{}

public:
	//~ Begin of FFastArraySerializer
	UE_API void PreReplicatedRemove(const TArrayView<int32>& RemovedIndices, int32 FinalSize);
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FUIFrameworkGameLayerSlot, FUIFrameworkGameLayerSlotList>(Entries, DeltaParms, *this);
	}

	UE_API void AddEntry(FUIFrameworkGameLayerSlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Layer);
	UE_API FUIFrameworkGameLayerSlot* FindEntry(FUIFrameworkWidgetId WidgetId);
	UE_API const FUIFrameworkGameLayerSlot* FindEntry(FUIFrameworkWidgetId WidgetId) const;

private:
	UPROPERTY()
	TArray<FUIFrameworkGameLayerSlot> Entries;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<UUIFrameworkPlayerComponent> Owner;
};

template<>
struct TStructOpsTypeTraits<FUIFrameworkGameLayerSlotList> : public TStructOpsTypeTraitsBase2<FUIFrameworkGameLayerSlotList>
{
	enum { WithNetDeltaSerializer = true };
};


/**
 * 
 */
UCLASS(MinimalAPI, Blueprintable, meta = (BlueprintSpawnableComponent))
class UUIFrameworkPlayerComponent : public UActorComponent, public IUIFrameworkWidgetTreeOwner
{
	GENERATED_BODY()

	friend FUIFrameworkGameLayerSlotList;

public:
	UE_API UUIFrameworkPlayerComponent();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void AddWidget(FUIFrameworkGameLayerSlot Widget);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void RemoveWidget(UUIFrameworkWidget* Widget);
	
	const FUIFrameworkGameLayerSlotList& GetRootList() const
	{
		return RootList;
	}

	/** Gets the controller that owns the component, this will always be valid during gameplay but can return null in the editor */
	template <class T = APlayerController>
	T* GetPlayerController() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerController>::Value, "'T' template parameter to GetPlayerController must be derived from APlayerController");
		AActor* Owner = GetOwner();
		while (Owner != nullptr)
		{
			if (T* PC = Cast<T>(Owner))
			{
				return PC;
			}

			Owner = Owner->GetOwner();
		}
		return nullptr;
	}

	template <class T = APlayerController>
	T* GetPlayerControllerChecked() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerController>::Value, "'T' template parameter to GetPlayerControllerChecked must be derived from APlayerController");
		T* Result = GetPlayerController<T>();
		check(Result);
		return Result;
	}

	//~ Begin UActorComponent
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void UninitializeComponent() override;
	UE_API virtual bool ReplicateSubobjects(class UActorChannel* Channel, class FOutBunch* Bunch, FReplicationFlags* RepFlags) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent

	UE_API void AuthorityRemoveChild(UUIFrameworkWidget* Widget);
	UE_API FOnPendingReplicationProcessed& GetOnPendingReplicationProcessed();

	UE_API virtual FUIFrameworkWidgetTree& GetWidgetTree() override;
	UE_API virtual FUIFrameworkWidgetOwner GetWidgetOwner() const override;
	UE_API virtual void LocalWidgetWasAddedToTree(const FUIFrameworkWidgetTreeEntry& Entry) override;
	UE_API virtual void LocalWidgetRemovedFromTree(const FUIFrameworkWidgetTreeEntry& Entry) override;
	UE_API virtual void LocalRemoveWidgetRootFromTree(const UUIFrameworkWidget* Widget) override;

	UE_API void SetWidgetToFocus(FUIFrameworkWidgetId WidgetId);

protected:
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId WidgetId);

private:
	UFUNCTION(Server, Reliable)
	UE_API void ServerRemoveWidgetRootFromTree(FUIFrameworkWidgetId WidgetId);

	UE_API void LocalOnClassLoaded(TSoftClassPtr<UWidget> WidgetClass);

	UFUNCTION()
	UE_API void OnRep_WidgetToFocus();
	UE_API bool TrySetFocus(float DeltaTime, FUIFrameworkWidgetId CurrentWidgetToFocusId);

	void HandleAddPending();

	UUIFrameworkPresenter* GetOrCreatePresenter();
	
private:
	UPROPERTY(Replicated)
	FUIFrameworkGameLayerSlotList RootList;

	UPROPERTY(Replicated)
	FUIFrameworkWidgetTree WidgetTree;

	UPROPERTY(Transient)
	TObjectPtr<UUIFrameworkPresenter> Presenter;

	//~ Widget can be net replicated but not constructed yet.
	UPROPERTY(Transient)
	TSet<int32> NetReplicationPending;

	//~ Widgets are created and ready to be added.
	UPROPERTY(Transient)
	TSet<int32> AddPending;

	//~ Once widgets are created and constructed, allow for actions such as focus to occur
	FOnPendingReplicationProcessed OnPendingReplicationProcessed;

	//~ Widget to focus on
	UPROPERTY(ReplicatedUsing=OnRep_WidgetToFocus, Transient)
	FUIFrameworkWidgetToFocus WidgetToFocus;

	bool bIsPendingFocusCall = false;

	struct FWidgetClassToLoad
	{
		TArray<int32, TInlineAllocator<4>> EntryReplicationIds;
		TSharedPtr<FStreamableHandle> StreamableHandle;
	};

	TMap<TSoftClassPtr<UWidget>, FWidgetClassToLoad> ClassesToLoad;

	bool bAddingWidget = false;

	FTSTicker::FDelegateHandle PendingWidgetConstructedTickerHandle;
};

#undef UE_API
