// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIFPlayerComponent.h"

#include "UIFPresenter.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkGameLayerSlot;
class FUIFrameworkModule;
class UWidget;

/**
 * 
 */
 UCLASS(MinimalAPI, Abstract, Within=UIFrameworkPlayerComponent)
class UUIFrameworkPresenter : public UObject
{
	GENERATED_BODY()

public:
	virtual void AddToViewport(UWidget* UMGWidget, const FUIFrameworkGameLayerSlot& Slot)
	{

	}
	virtual void RemoveFromViewport(FUIFrameworkWidgetId WidgetId)
	{

	}
	virtual void FocusWidget(UWidget* UMGWidget)
	{

	}
};


/**
 *
 */
 UCLASS(MinimalAPI)
class UUIFrameworkGameViewportPresenter : public UUIFrameworkPresenter
 {
	 GENERATED_BODY()

public:
	UE_API virtual void AddToViewport(UWidget* UMGWidget, const FUIFrameworkGameLayerSlot& Slot) override;
	UE_API virtual void RemoveFromViewport(FUIFrameworkWidgetId WidgetId) override;
	UE_API virtual void FocusWidget(UWidget* UMGWidget) override;
	UE_API virtual void BeginDestroy() override;

private:
	struct FWidgetPair
	{
		FWidgetPair() = default;
		FWidgetPair(UWidget* Widget, FUIFrameworkWidgetId WidgetId);
		TWeakObjectPtr<UWidget> UMGWidget;
		FUIFrameworkWidgetId WidgetId;
	};
	TArray<FWidgetPair> Widgets;
};

#undef UE_API
