// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "MVVMViewModelBase.h"
#include "Types/UIFParentWidget.h"
#include "Types/UIFWidgetId.h"

#include "UIFWidget.generated.h"

#define UE_API UIFRAMEWORK_API

struct FStreamableHandle;
template <typename ObjectType> class TNonNullPtr;

class FObjectInitializer;
class FUIFrameworkModule;
class IUIFrameworkWidgetTreeOwner;
class UUIFrameworkWidget;
struct FUIFrameworkWidgetTree;

/**
 *
 */
UINTERFACE(MinimalAPI)
class UUIFrameworkWidgetWrapperInterface : public UInterface
{
	GENERATED_BODY()
};

class IUIFrameworkWidgetWrapperInterface
{
	GENERATED_BODY()
};

/**
 * 
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UUIFrameworkWidget : public UMVVMViewModelBase
{
	GENERATED_BODY()

	friend FUIFrameworkModule;
	friend FUIFrameworkWidgetTree;

public:
	UE_API UUIFrameworkWidget();
	UE_API UUIFrameworkWidget(const FObjectInitializer& ObjectInitializer);

private:
	UPROPERTY(BlueprintReadWrite, ReplicatedUsing = "OnRep_IsEnabled", Getter = "IsEnabled", Setter="SetEnabled", Category = "UI Framework", meta = (AllowPrivateAccess = "true"))
	bool bIsEnabled = true;

	UPROPERTY(BlueprintReadWrite, ReplicatedUsing = "OnRep_Visibility", Getter, Setter = "SetVisibility", Category = "UI Framework", meta = (AllowPrivateAccess = "true"))
	ESlateVisibility Visibility = ESlateVisibility::Visible;

	UPROPERTY(BlueprintReadWrite, Getter = "IsHitTestVisible", Setter = "SetHitTestVisible", Category = "UI Framework", meta = (AllowPrivateAccess = "true"))
	bool bIsHitTestVisible = true;

	UPROPERTY(BlueprintReadWrite, ReplicatedUsing = "OnRep_RenderOpacity", Getter, Setter, Category = "UI Framework", meta = (AllowPrivateAccess = "true"))
	double RenderOpacity = 1.0;

public:
	//~ Begin UObject
	virtual bool IsSupportedForNetworking() const override
	{
		return true;
	}
	UE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	UE_API virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;
	//~ End UObject

	// Gets the controller that owns the widget
	template <class T = APlayerController>
	T* GetPlayerController() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, APlayerController>::Value, "'T' template parameter to GetPlayerController must be derived from APlayerController");
		UObject* TestOuter = GetOuter();
		while (TestOuter != nullptr)
		{
			if (T* PC = Cast<T>(TestOuter))
			{
				return PC;
			}

			const AActor* OuterActor = Cast<AActor>(TestOuter);
			if (OuterActor && OuterActor->GetOwner())
			{
				TestOuter = OuterActor->GetOwner();
			}
			else
			{
				TestOuter = TestOuter->GetOuter();
			}
		}
		return nullptr;
	}

	TScriptInterface<IUIFrameworkWidgetWrapperInterface> AuthorityGetWrapper() const
	{
		return AuthorityWrapper;
	}

	void AuthoritySetWrapper(TScriptInterface<IUIFrameworkWidgetWrapperInterface> InWrapper)
	{
		AuthorityWrapper = InWrapper;
	}

	FUIFrameworkWidgetId GetWidgetId() const
	{
		return Id;
	}

	IUIFrameworkWidgetTreeOwner* GetWidgetTreeOwner() const
	{
		return WidgetTreeOwner;
	}

	UE_API FUIFrameworkWidgetTree* GetWidgetTree() const;

	TSoftClassPtr<UWidget> GetUMGWidgetClass() const
	{
		return WidgetClass;
	}

	//~ Authority functions
	FUIFrameworkParentWidget AuthorityGetParent() const
	{
		return AuthorityParent;
	}

	virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func)
	{
	}

	//~ Local functions
	virtual bool LocalIsReplicationReady() const
	{
		return true;
	}

	UWidget* LocalGetUMGWidget() const
	{
		return LocalUMGWidget;
	}

	// Can be used instead of manually calling LocalIsReplicationReady() and AsyncLoadWidgetClass()
	UE_API UWidget* LocalGetOrCreateUMGWidgetIfReady();

	UE_API TSharedPtr<FStreamableHandle> AsyncLoadWidgetClass();

	UE_API void LocalCreateUMGWidget(IUIFrameworkWidgetTreeOwner* Owner, bool* bDidCreateWidgetPtr = nullptr);
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId);
	UE_API void LocalDestroyUMGWidget();

	//~ Properties
	UE_API ESlateVisibility GetVisibility() const;
	UE_API void SetVisibility(ESlateVisibility InVisibility);

	UE_API bool IsEnabled() const;
	UE_API void SetEnabled(bool bEnabled);

	UE_API bool IsHitTestVisible() const;
	UE_API void SetHitTestVisible(bool bInHitTestVisible);

	UE_API double GetRenderOpacity() const;
	UE_API void SetRenderOpacity(double InRenderOpacity);

protected:
	virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget)
	{
	}
	virtual void AuthorityOnWidgetTreeOwnerChanged()
	{
	}
	virtual void LocalOnUMGWidgetCreated()
	{
	}

	UE_API void ForceNetUpdate();

private:
	UFUNCTION()
	UE_API void OnRep_IsEnabled();
	UFUNCTION()
	UE_API void OnRep_Visibility();
	UFUNCTION()
	UE_API void OnRep_RenderOpacity();

	void AuthoritySetWidgetTreeOwner(IUIFrameworkWidgetTreeOwner* InWidgetTreeOwner)
	{
		if (WidgetTreeOwner != InWidgetTreeOwner)
		{
			WidgetTreeOwner = InWidgetTreeOwner;
			AuthorityOnWidgetTreeOwnerChanged();
		}
	}

protected:
	UPROPERTY(BlueprintReadOnly, Replicated, EditDefaultsOnly, Category = "UI Framework")
	TSoftClassPtr<UWidget> WidgetClass; // todo: make this private and use a constructor argument

private:
	//~ Authority and Local
	UPROPERTY(Replicated, Transient, DuplicateTransient)
	FUIFrameworkWidgetId Id;

	//~ Authority
	UPROPERTY(Transient)
	TScriptInterface<IUIFrameworkWidgetWrapperInterface> AuthorityWrapper;

	//~ Authority and Local
	IUIFrameworkWidgetTreeOwner* WidgetTreeOwner = nullptr;

	//~ AuthorityOnly
	UPROPERTY(Transient)
	FUIFrameworkParentWidget AuthorityParent;
	
	//~ LocalOnly
	UPROPERTY(Transient)
	TObjectPtr<UWidget> LocalUMGWidget;

	TSharedPtr<FStreamableHandle> WidgetClassStreamableHandle;
};

#undef UE_API
