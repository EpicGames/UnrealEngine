// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/UIFSlotBase.h"
#include "Types/UIFEvents.h"
#include "UIFWidget.h"

#include "UIFButton.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;
enum class ECommonInputType : uint8;
class UInputAction;

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "Button UIFramework")
class UUIFrameworkButton : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkButton();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetContent(FUIFrameworkSimpleSlot Content);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FUIFrameworkSimpleSlot GetContent() const
	{
		return Slot;
	}

	UE_API void SetTriggeringInputAction(TSoftObjectPtr<UInputAction> InputAction);

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;

protected:
	UE_API virtual void LocalOnUMGWidgetCreated() override;

private:
	UFUNCTION()
	UE_API void HandleClick();

	UFUNCTION(Server, Reliable)
	UE_API void ServerClick(APlayerController* PlayerController);

	UFUNCTION()
	UE_API void HandleHighlight();

	UFUNCTION(Server, Reliable)
	UE_API void ServerHighlight(APlayerController* PlayerController);

	UFUNCTION()
	UE_API void HandleUnhighlight();

	UFUNCTION(Server, Reliable)
	UE_API void ServerUnhighlight(APlayerController* PlayerController);

	UFUNCTION()
	UE_API void HandleHover();

	UFUNCTION()
	UE_API void HandleUnhover();

	UFUNCTION()
	UE_API void OnRep_Slot();

	UFUNCTION()
	UE_API void OnRep_TriggeringInputAction();

	void LoadInputAction();
	void HandleInputActionLoaded();

public:
	FUIFrameworkClickEvent OnClick;

	FUIFrameworkHighlightEvent OnHighlight;

	FUIFrameworkUnhighlightEvent OnUnhighlight;

private:
	UPROPERTY(/*ExposeOnSpawn, */ReplicatedUsing = OnRep_Slot)
	FUIFrameworkSimpleSlot Slot;

	bool bIsHighlighted = false;

	TSharedPtr<FStreamableHandle> AsyncLoadInputActionHandle;

	UPROPERTY(ReplicatedUsing = OnRep_TriggeringInputAction)
	TSoftObjectPtr<UInputAction> TriggeringInputAction;
};

#undef UE_API
