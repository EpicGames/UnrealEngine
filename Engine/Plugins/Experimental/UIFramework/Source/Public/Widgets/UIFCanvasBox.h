// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/UIFSlotBase.h"
#include "UIFWidget.h"
#include "Widgets/Layout/Anchors.h"

#include "UIFCanvasBox.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;

class UUIFrameworkCanvasBox;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkCanvasBoxSlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	/** Anchors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Canvas)
	FAnchors Anchors = FAnchors(0.0f, 0.0f);

	/** Offset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Canvas)
	FMargin Offsets = FMargin(0.f, 0.f, 0.f, 0.f);

	/**
	 * Alignment is the pivot point of the widget.  Starting in the upper left at (0,0),
	 * ending in the lower right at (1,1).  Moving the alignment point allows you to move
	 * the origin of the widget.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Canvas)
	FVector2D Alignment = FVector2D::ZeroVector;

	/** The order priority this widget is rendered inside the layer. Higher values are rendered last (and so they will appear to be on top). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Canvas)
	int32 ZOrder = 0;

	/** When true we use the widget's desired size */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Canvas)
	bool bSizeToContent = false;
};


/**
 *
 */
USTRUCT()
struct FUIFrameworkCanvasBoxSlotList : public FFastArraySerializer
{
	GENERATED_BODY()

	FUIFrameworkCanvasBoxSlotList() = default;
	FUIFrameworkCanvasBoxSlotList(UUIFrameworkCanvasBox* InOwner)
		: Owner(InOwner)
	{}

public:
	//~ Begin of FFastArraySerializer
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms);

	UE_API void AddEntry(FUIFrameworkCanvasBoxSlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkCanvasBoxSlot* FindEntry(FUIFrameworkWidgetId WidgetId);
	UE_API void ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func);

private:
	UPROPERTY()
	TArray<FUIFrameworkCanvasBoxSlot> Slots;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<UUIFrameworkCanvasBox> Owner;
};


template<>
struct TStructOpsTypeTraits<FUIFrameworkCanvasBoxSlotList> : public TStructOpsTypeTraitsBase2<FUIFrameworkCanvasBoxSlotList>
{
	enum { WithNetDeltaSerializer = true };
};


/**
 *
 */
UCLASS(MinimalAPI, DisplayName="Canvas UIFramework")
class UUIFrameworkCanvasBox : public UUIFrameworkWidget
{
	GENERATED_BODY()

	friend FUIFrameworkCanvasBoxSlotList;

public:
	UE_API UUIFrameworkCanvasBox();

public:
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void AddWidget(FUIFrameworkCanvasBoxSlot Widget);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void RemoveWidget(UUIFrameworkWidget* Widget);

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;

private:
	UE_API void AddEntry(FUIFrameworkCanvasBoxSlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkCanvasBoxSlot* FindEntry(FUIFrameworkWidgetId WidgetId);

private:
	UPROPERTY(Replicated)
	FUIFrameworkCanvasBoxSlotList ReplicatedSlotList;
};

#undef UE_API
