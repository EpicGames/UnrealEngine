// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIFWidget.h"

#include "UIFColorBlock.generated.h"

#define UE_API UIFRAMEWORK_API

class UImage;
class UMaterialInterface;
struct FStreamableHandle;
class UTexture2D;

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "Color Block UIFramework")
class UUIFrameworkColorBlock : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkColorBlock();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetColor(FLinearColor Tint);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FLinearColor GetColor() const
	{
		return Color;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetDesiredSize(FVector2f DesiredSize);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FVector2f GetDesiredSize() const
	{
		return DesiredSize;
	}
	
	UE_API virtual void LocalOnUMGWidgetCreated() override;

private:
	UFUNCTION()
	UE_API void OnRep_Color();
	
	UFUNCTION()
	UE_API void OnRep_DesiredSize();

private:
	UPROPERTY(ReplicatedUsing = OnRep_Color)
	FLinearColor Color;
	
	UPROPERTY(ReplicatedUsing = OnRep_DesiredSize)
	FVector2f DesiredSize;
};

#undef UE_API
