// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIFWidget.h"

#include "UIFImageBlock.generated.h"

#define UE_API UIFRAMEWORK_API

class UImage;
class UMaterialInterface;
struct FStreamableHandle;
class UTexture2D;

/**
 * 
 */
USTRUCT()
struct FUIFrameworkImageBlockData
{
	GENERATED_BODY()

	/** Tinting applied to the image. */
	UPROPERTY()
	FLinearColor Tint = FLinearColor::White;

	/**
	 * The image to render for this brush, can be a UTexture2D or UMaterialInterface or an object implementing
	 * the AtlasedTextureInterface.
	 */
	UPROPERTY()
	TSoftObjectPtr<UObject> ResourceObject;

	/** Size of the resource in Slate Units */
	UPROPERTY()
	FVector2f DesiredSize = FVector2f::ZeroVector;

	/** How to tile the image. */
	UPROPERTY()
	TEnumAsByte<ESlateBrushTileType::Type> Tiling = ESlateBrushTileType::NoTile;

	UPROPERTY()
	bool bUseTextureSize = false;
};

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "Image Block UIFramework")
class UUIFrameworkImageBlock : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkImageBlock();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API virtual void SetMaterial(TSoftObjectPtr<UMaterialInterface> Material);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	TSoftObjectPtr<UObject> GeResourceObject() const
	{
		return Data.ResourceObject;
	}
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetTexture(TSoftObjectPtr<UTexture2D> Texture, bool bUseTextureSize);
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetTint(FLinearColor Tint);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FLinearColor GetTint() const
	{
		return Data.Tint;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetDesiredSize(FVector2f DesiredSize);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FVector2f GetDesiredSize() const
	{
		return Data.DesiredSize;
	}
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetTiling(ESlateBrushTileType::Type OverflowPolicy);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	ESlateBrushTileType::Type GetTiling() const
	{
		return Data.Tiling;
	}

	UE_API virtual void LocalOnUMGWidgetCreated() override;
	UE_API virtual bool LocalIsReplicationReady() const override;

protected:
	UE_API virtual UObject* ProcessResourceObject();

private:
	UFUNCTION()
	UE_API void OnRep_Data();

private:
	/** Tinting applied to the image. */
	UPROPERTY(ReplicatedUsing = OnRep_Data)
	FUIFrameworkImageBlockData Data;

	UPROPERTY()
	bool bWaitForResourceToBeLoaded = false;

	TSharedPtr<FStreamableHandle> StreamingHandle;
	int32 StreamingCounter = 0;
};

#undef UE_API
