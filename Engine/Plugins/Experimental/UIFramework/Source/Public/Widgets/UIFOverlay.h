// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/UIFSlotBase.h"
#include "UIFWidget.h"

#include "UIFOverlay.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;

class UUIFrameworkOverlay;
struct FUIFrameworkOverlaySlotList;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkOverlaySlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	friend UUIFrameworkOverlay;
	friend FUIFrameworkOverlaySlotList;

	/** Distance between that surrounds the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	FMargin Padding = FMargin(0.f, 0.f, 0.f, 0.f);

	/** Horizontal alignment of the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;

	/** Vertical alignment of the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = EVerticalAlignment::VAlign_Fill;

private:
	/** Index in the array the Slot is. The position in the array can change when replicated. */
	UPROPERTY()
	int32 Index = INDEX_NONE;
};


/**
 *
 */
USTRUCT()
struct FUIFrameworkOverlaySlotList : public FFastArraySerializer
{
	GENERATED_BODY()

	FUIFrameworkOverlaySlotList() = default;
	FUIFrameworkOverlaySlotList(UUIFrameworkOverlay* InOwner)
		: Owner(InOwner)
	{}

public:
	//~ Begin of FFastArraySerializer
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms);

	UE_API void AddEntry(FUIFrameworkOverlaySlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkOverlaySlot* FindEntry(FUIFrameworkWidgetId WidgetId);
	UE_API void ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func);

private:
	UPROPERTY()
	TArray<FUIFrameworkOverlaySlot> Slots;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<UUIFrameworkOverlay> Owner;
};


template<>
struct TStructOpsTypeTraits<FUIFrameworkOverlaySlotList> : public TStructOpsTypeTraitsBase2<FUIFrameworkOverlaySlotList>
{
	enum { WithNetDeltaSerializer = true };
};


/**
 *
 */
UCLASS(MinimalAPI, DisplayName="Overlay UIFramework")
class UUIFrameworkOverlay : public UUIFrameworkWidget
{
	GENERATED_BODY()

	friend FUIFrameworkOverlaySlotList;

public:
	UE_API UUIFrameworkOverlay();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void AddWidget(FUIFrameworkOverlaySlot Widget);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void RemoveWidget(UUIFrameworkWidget* Widget);

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;

private:
	UE_API void AddEntry(FUIFrameworkOverlaySlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkOverlaySlot* FindEntry(FUIFrameworkWidgetId WidgetId);

	UPROPERTY(Replicated)
	FUIFrameworkOverlaySlotList ReplicatedSlotList;
};

#undef UE_API
