// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/UIFSlotBase.h"
#include "UIFWidget.h"

#include "UIFSafeZoneBox.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "SafeZone UIFramework")
class UUIFrameworkSafeZoneBox : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkSafeZoneBox();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetContent(FUIFrameworkSlotBase Content);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FUIFrameworkSlotBase GetContent() const
	{
		return Slot;
	}

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;

private:
	UPROPERTY(Replicated)
	FUIFrameworkSlotBase Slot;
};

#undef UE_API
