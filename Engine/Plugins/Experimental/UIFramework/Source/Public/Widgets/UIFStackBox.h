// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/UIFSlotBase.h"
#include "UIFWidget.h"

#include "UIFStackBox.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;

class UUIFrameworkStackBox;
struct FUIFrameworkStackBoxSlotList;

/**
 *
 */
USTRUCT(BlueprintType)
struct FUIFrameworkStackBoxSlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	friend UUIFrameworkStackBox;
	friend FUIFrameworkStackBoxSlotList;

	/** Horizontal alignment of the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;

	/** Vertical alignment of the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = EVerticalAlignment::VAlign_Fill;

	/** Distance between that surrounds the widget inside the slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	FMargin Padding = FMargin(0.f, 0.f, 0.f, 0.f);

	/** How much space this slot should occupy in the direction of the panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	FSlateChildSize Size = FSlateChildSize(ESlateSizeRule::Automatic);

private:
	/** Index in the array the Slot is. The position in the array can change when replicated. */
	UPROPERTY()
	int32 Index = INDEX_NONE;
};


/**
 *
 */
USTRUCT()
struct FUIFrameworkStackBoxSlotList : public FFastArraySerializer
{
	GENERATED_BODY()

	FUIFrameworkStackBoxSlotList() = default;
	FUIFrameworkStackBoxSlotList(UUIFrameworkStackBox* InOwner)
		: Owner(InOwner)
	{}

public:
	//~ Begin of FFastArraySerializer
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms);

	UE_API void AddEntry(FUIFrameworkStackBoxSlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkStackBoxSlot* FindEntry(FUIFrameworkWidgetId WidgetId);
	UE_API void ForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func);

private:
	UPROPERTY()
	TArray<FUIFrameworkStackBoxSlot> Slots;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<UUIFrameworkStackBox> Owner;
};


template<>
struct TStructOpsTypeTraits<FUIFrameworkStackBoxSlotList> : public TStructOpsTypeTraitsBase2<FUIFrameworkStackBoxSlotList>
{
	enum { WithNetDeltaSerializer = true };
};


/**
 *
 */
UCLASS(MinimalAPI, DisplayName="StackBox UIFramework")
class UUIFrameworkStackBox : public UUIFrameworkWidget
{
	GENERATED_BODY()

	friend FUIFrameworkStackBoxSlotList;

public:
	UE_API UUIFrameworkStackBox();

private:
	/** The orientation of the stack box. */
	UPROPERTY(ReplicatedUsing="OnRep_Orientation", EditAnywhere, BlueprintReadWrite, Setter, Getter, Category = "UI Framework", meta = (AllowPrivateAccess = "true"))
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Horizontal;

public:
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void AddWidget(FUIFrameworkStackBoxSlot Widget);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void RemoveWidget(UUIFrameworkWidget* Widget);

	/** Get the orientation of the stack box. */
	UE_API EOrientation GetOrientation() const;
	/** Set the orientation of the stack box. The existing elements will be rearranged. */
	UE_API void SetOrientation(EOrientation Value);

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;
	UE_API virtual void LocalOnUMGWidgetCreated() override;

private:
	UE_API void AddEntry(FUIFrameworkStackBoxSlot Entry);
	UE_API bool RemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkStackBoxSlot* FindEntry(FUIFrameworkWidgetId WidgetId);

	UFUNCTION()
	UE_API void OnRep_Orientation();

private:
	UPROPERTY(Replicated)
	FUIFrameworkStackBoxSlotList ReplicatedSlotList;
};

#undef UE_API
