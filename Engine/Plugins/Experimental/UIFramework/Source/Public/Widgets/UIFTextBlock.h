// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "UIFWidget.h"

#include "LocalizableMessage.h"

#include "UIFTextBlock.generated.h"

#define UE_API UIFRAMEWORK_API

namespace ETextJustify { enum Type : int; }

class UTextBlock;

/**
 *
 */
UCLASS(MinimalAPI, Abstract, HideDropDown)
class UUIFrameworkTextBase : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkTextBase();

	UE_API void SetMessage(FLocalizableMessage&& InMessage);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FText GetText() const
	{
		return Text;
	}
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetTextColor(FLinearColor TextColor);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FLinearColor GetTextColor() const
	{
		return TextColor;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetTextSize(float TextSize);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	float GetTextSize() const
	{
		return TextSize;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetAutoWrap(bool InAutoWrap);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	bool GetAutoWrap() const
	{
		return AutoWrap;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetWrapWidth(float InWrapWidth);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	float GetWrapWidth() const
	{
		return WrapWidth;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	ETextWrappingPolicy GetWrappingPolicy() const
	{
		return WrappingPolicy;
	}

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetJustification(ETextJustify::Type Justification);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	ETextJustify::Type GetJustification() const
	{
		return Justification;
	}
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetOverflowPolicy(ETextOverflowPolicy OverflowPolicy);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	ETextOverflowPolicy GetOverflowPolicy() const
	{
		return OverflowPolicy;
	}

	UE_API virtual void LocalOnUMGWidgetCreated() override;

private:
	UFUNCTION()
	UE_API void OnRep_Message();
	
	UFUNCTION()
	UE_API void OnRep_TextColor();

	UFUNCTION()
	UE_API void OnRep_TextSize();

	UFUNCTION()
	UE_API void OnRep_AutoWrap();

	UFUNCTION()
	UE_API void OnRep_WrapWidth();

	UFUNCTION()
	UE_API void OnRep_WrappingPolicy();

	UFUNCTION()
	UE_API void OnRep_Justification();

	UFUNCTION()
	UE_API void OnRep_OverflowPolicy();

	UE_API void SetText(const FLocalizableMessage& InMessage);

protected:
	virtual void SetTextToWidget(const FText&)
	{
	}
	virtual void SetTextColorToWidget(FLinearColor)
	{
	}
	virtual void SetTextSizeToWidget(float)
	{
	}
	virtual void SetAutoWrapToWidget(bool)
	{
	}
	virtual void SetWrapWidthToWidget(float)
	{
	}
	virtual void SetWrappingPolicyToWidget(ETextWrappingPolicy)
	{
	}
	virtual void SetJustificationToWidget(ETextJustify::Type)
	{
	}
	virtual void SetOverflowPolicyToWidget(ETextOverflowPolicy)
	{
	}

private:
	UPROPERTY(Transient)
	FText Text;

	UPROPERTY(ReplicatedUsing = OnRep_Message)
	FLocalizableMessage Message;

	UPROPERTY(ReplicatedUsing = OnRep_TextColor)
	FLinearColor TextColor = FLinearColor::Black;

	UPROPERTY(ReplicatedUsing = OnRep_TextSize)
	float TextSize = 24.0;

	UPROPERTY(ReplicatedUsing = OnRep_AutoWrap)
	bool AutoWrap = false;

	UPROPERTY(ReplicatedUsing = OnRep_WrapWidth)
	float WrapWidth = 0.0f;

	UPROPERTY(ReplicatedUsing = OnRep_WrappingPolicy)
	ETextWrappingPolicy WrappingPolicy;

	UPROPERTY(ReplicatedUsing = OnRep_Justification)
	TEnumAsByte<ETextJustify::Type> Justification;

	UPROPERTY(ReplicatedUsing = OnRep_OverflowPolicy)
	ETextOverflowPolicy OverflowPolicy = ETextOverflowPolicy::Clip;
};

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "Text Block UIFramework")
class UUIFrameworkTextBlock : public UUIFrameworkTextBase
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkTextBlock();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetShadowOffset(FVector2f ShadowOffset);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FVector2f GetShadowOffset() const
	{
		return ShadowOffset;
	}
	
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetShadowColor(FLinearColor ShadowColor);

	UFUNCTION(BlueprintCallable, Category = "UI Framework")
	FLinearColor GetShadowColor() const
	{
		return ShadowColor;
	}

	UE_API virtual void LocalOnUMGWidgetCreated() override;

private:
	UFUNCTION()
	UE_API void OnRep_ShadowOffset();
	UFUNCTION()
	UE_API void OnRep_ShadowColor();

protected:
	UE_API virtual void SetTextToWidget(const FText&) override;
	UE_API virtual void SetJustificationToWidget(ETextJustify::Type) override;
	UE_API virtual void SetOverflowPolicyToWidget(ETextOverflowPolicy) override;
	UE_API virtual void SetTextColorToWidget(FLinearColor) override;
	UE_API virtual void SetTextSizeToWidget(float) override;
	UE_API virtual void SetAutoWrapToWidget(bool) override;
	UE_API virtual void SetWrapWidthToWidget(float) override;
	UE_API virtual void SetWrappingPolicyToWidget(ETextWrappingPolicy) override;
	UE_API virtual void SetShadowOffsetToWidget(FVector2f);
	UE_API virtual void SetShadowColorToWidget(FLinearColor);

private:
	UPROPERTY(ReplicatedUsing = OnRep_ShadowOffset)
	FVector2f ShadowOffset = FVector2f::ZeroVector;
	
	UPROPERTY(ReplicatedUsing = OnRep_ShadowColor)
	FLinearColor ShadowColor = FLinearColor::Black;
};

#undef UE_API
