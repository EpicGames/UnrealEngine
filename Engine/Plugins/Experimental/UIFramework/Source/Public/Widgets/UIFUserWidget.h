// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INotifyFieldValueChanged.h"
#include "UIFWidget.h"
#include "Types/UIFSlotBase.h"

#include "UIFUserWidget.generated.h"

#define UE_API UIFRAMEWORK_API

struct FUIFrameworkWidgetId;

class UUserWidget;
class UUIFrameworkUserWidget;

/**
 *
 */
USTRUCT()
struct FUIFrameworkUserWidgetNamedSlot : public FUIFrameworkSlotBase
{
	GENERATED_BODY()

	/** The name of the NamedSlot */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Framework")
	FName SlotName;
};

/**
 *
 */
USTRUCT()
struct FUIFrameworkUserWidgetNamedSlotList : public FFastArraySerializer
{
	GENERATED_BODY()

	FUIFrameworkUserWidgetNamedSlotList() = default;
	FUIFrameworkUserWidgetNamedSlotList(UUIFrameworkUserWidget* InOwner)
		: Owner(InOwner)
	{}

public:
	//~ Begin of FFastArraySerializer
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	//~ End of FFastArraySerializer

	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms);

	UE_API void AuthorityAddEntry(FUIFrameworkUserWidgetNamedSlot Entry);
	UE_API bool AuthorityRemoveEntry(UUIFrameworkWidget* Widget);
	UE_API FUIFrameworkUserWidgetNamedSlot* FindEntry(FUIFrameworkWidgetId WidgetId);
	UE_API const FUIFrameworkUserWidgetNamedSlot* AuthorityFindEntry(FName SlotName) const;
	UE_API void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func);

private:
	UPROPERTY()
	TArray<FUIFrameworkUserWidgetNamedSlot> Slots;

	UPROPERTY(NotReplicated, Transient)
	TObjectPtr<UUIFrameworkUserWidget> Owner;
};


template<>
struct TStructOpsTypeTraits<FUIFrameworkUserWidgetNamedSlotList> : public TStructOpsTypeTraitsBase2<FUIFrameworkUserWidgetNamedSlotList>
{
	enum { WithNetDeltaSerializer = true };
};

/**
 *
 */
UCLASS(MinimalAPI, DisplayName = "UserWidget UIFramework")
class UUIFrameworkUserWidget : public UUIFrameworkWidget
{
	GENERATED_BODY()

public:
	UE_API UUIFrameworkUserWidget();

public:
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetWidgetClass(TSoftClassPtr<UWidget> Value);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API void SetNamedSlot(FName SlotName, UUIFrameworkWidget* Widget);

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "UI Framework")
	UE_API UUIFrameworkWidget* GetNamedSlot(FName SlotName) const;

public:
	UE_API virtual bool LocalIsReplicationReady() const override;

	UE_API virtual void AuthorityForEachChildren(const TFunctionRef<void(UUIFrameworkWidget*)>& Func) override;
	UE_API virtual void AuthorityRemoveChild(UUIFrameworkWidget* Widget) override;
	UE_API virtual void LocalAddChild(FUIFrameworkWidgetId ChildId) override;

private:
	UPROPERTY(Replicated)
	FUIFrameworkUserWidgetNamedSlotList ReplicatedNamedSlotList;
};

#undef UE_API
