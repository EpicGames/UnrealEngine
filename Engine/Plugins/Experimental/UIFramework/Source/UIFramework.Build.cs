// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UIFramework : ModuleRules
{
	public UIFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CommonUI",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EnhancedInput",
				"FieldNotification",
				"LocalizableMessage",
				"ModelViewViewModel",
				"SlateCore",
				"Slate",
				"UMG",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"NetCore",
				"CommonInput",
				"GameplayTags",
			}
		);

		SetupIrisSupport(Target);
	}
}
