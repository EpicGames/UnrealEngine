{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "UIFramework",
	"Description": "A framework to control UMG from server.",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "UIFramework",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ModelViewViewModel",
			"Enabled": true
		},
		{
			"Name": "LocalizableMessage",
			"Enabled": true
		},
		{
			"Name": "CommonUI",
			"Enabled": true
		}
	]
}